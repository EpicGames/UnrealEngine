// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class Merge : ModuleRules
	{
        public Merge(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
				    "AssetTools",
					"Core",
				    "CoreUObject",
					"EditorFramework",
				    
				    "Engine", // needed so that we can clone blueprints...
				    "GraphEditor",
				    "InputCore",
				    "Kismet",
					"PropertyEditor",
				    "Slate",
				    "SlateCore",
				    "SourceControl",
				    "UnrealEd",
				}
			);
		}
	}
}
