// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "IAssetTypeActions.h"

class FBlueprintEditor;

// This data is shared by the various controls (graph view, components view, and defaults view) 
// that are presented as part of the merge tool. Each view keeps its own shallow copy of the data:
struct FBlueprintMergeData
{
	FBlueprintMergeData()
		: OwningEditor()
		, BlueprintLocal(NULL)
		, RevisionLocal(FRevisionInfo::InvalidRevision())
		, BlueprintBase(NULL)
		, RevisionBase(FRevisionInfo::InvalidRevision())
		, BlueprintRemote(NULL)
		, RevisionRemote(FRevisionInfo::InvalidRevision())
	{
	}

	FBlueprintMergeData( 
		TWeakPtr<class FBlueprintEditor>	InOwningEditor
		, const class UBlueprint*			InBlueprintLocal
		, const class UBlueprint*			InBlueprintBase
		, FRevisionInfo						InRevisionBase
		, const class UBlueprint*			InBlueprintRemote
		, FRevisionInfo						InRevisionRemote
	)
		: OwningEditor(		InOwningEditor		)
		, BlueprintLocal(	InBlueprintLocal	)
		, RevisionLocal(	FRevisionInfo::InvalidRevision())
		, BlueprintBase(	InBlueprintBase		)
		, RevisionBase(		InRevisionBase		)
		, BlueprintRemote(	InBlueprintRemote	)
		, RevisionRemote(	InRevisionRemote	)
	{
	}

	TWeakPtr<class FBlueprintEditor>	OwningEditor;
	
	const class UBlueprint*				BlueprintLocal;
	FRevisionInfo						RevisionLocal;

	const class UBlueprint*				BlueprintBase;
	FRevisionInfo						RevisionBase;

	const class UBlueprint*				BlueprintRemote;
	FRevisionInfo						RevisionRemote;
};

namespace EMergeParticipant
{
	enum Type
	{
		Remote,
		Base,
		Local,

		Max_None,
	};
}

DECLARE_DELEGATE(FOnMergeNodeSelected);

