// Copyright Epic Games, Inc. All Rights Reserved.

#include "Merge.h"
#include "Engine/Blueprint.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Framework/Docking/TabManager.h"
#include "ISourceControlState.h"
#include "ISourceControlRevision.h"
#include "ISourceControlProvider.h"
#include "ISourceControlModule.h"

#include "IAssetTypeActions.h"
#include "BlueprintMergeData.h"
#include "MergeUtils.h"
#include "UObject/Package.h"
#include "BlueprintEditor.h"
#include "SBlueprintMerge.h"
#include "Widgets/Docking/SDockTab.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "Merge"

const FName MergeToolTabId = FName(TEXT("MergeTool"));

static void DisplayErrorMessage( const FText& ErrorMessage )
{
	FNotificationInfo Info(ErrorMessage);
	Info.ExpireDuration = 5.0f;
	FSlateNotificationManager::Get().AddNotification(Info);
}

static FRevisionInfo GetRevisionInfo(ISourceControlRevision const& FromRevision)
{
	FRevisionInfo Ret = { FromRevision.GetRevision(), FromRevision.GetCheckInIdentifier(), FromRevision.GetDate() };
	return Ret;
}

static const UObject* LoadHeadRev(const FString& PackageName, const FString& AssetName, const ISourceControlState& SourceControlState, FRevisionInfo& OutRevInfo)
{
	// HistoryItem(0) is apparently the head revision:
	TSharedPtr<ISourceControlRevision, ESPMode::ThreadSafe> Revision = SourceControlState.GetHistoryItem(0);
	check(Revision.IsValid());
	OutRevInfo = GetRevisionInfo(*Revision);
	return FMergeToolUtils::LoadRevision(AssetName, *Revision);
}

static const UObject* LoadBaseRev(const FString& PackageName, const FString& AssetName, const ISourceControlState& SourceControlState, FRevisionInfo& OutRevInfo)
{
	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	const ISourceControlState::FResolveInfo ResolveInfo = SourceControlState.GetResolveInfo();
	if (ResolveInfo.IsValid())
	{
		const FSourceControlStatePtr BaseBranch = SourceControlProvider.GetState(ResolveInfo.BaseFile, EStateCacheUsage::Use);
		if (BaseBranch.IsValid())
		{
			const TSharedPtr<ISourceControlRevision, ESPMode::ThreadSafe> Revision = BaseBranch->FindHistoryRevision(ResolveInfo.BaseRevision);
			if (Revision.IsValid())
			{
				OutRevInfo = GetRevisionInfo(*Revision);
				return FMergeToolUtils::LoadRevision(AssetName, *Revision);
			}
		}
	}
	
	OutRevInfo = FRevisionInfo::InvalidRevision();
	return nullptr;
}

static TSharedPtr<SWidget> GenerateMergeTabContents(TSharedRef<FBlueprintEditor> Editor,
                                                    const UBlueprint*       BaseBlueprint,
                                                    const FRevisionInfo&    BaseRevInfo,
													const UBlueprint*       RemoteBlueprint,
													const FRevisionInfo&    RemoteRevInfo,
													const UBlueprint*       LocalBlueprint,
													const FOnMergeResolved& MereResolutionCallback)
{
	bool bForceAssetPicker = false;
	if (BaseBlueprint == nullptr)
	{
		BaseBlueprint = LocalBlueprint;
		bForceAssetPicker = true;
	}
	if (RemoteBlueprint == nullptr)
	{
		RemoteBlueprint = LocalBlueprint;
		bForceAssetPicker = true;
	}

	FBlueprintMergeData Data(Editor
		, LocalBlueprint
		, BaseBlueprint
		, BaseRevInfo
		, RemoteBlueprint
		, RemoteRevInfo);

	return SNew(SBlueprintMerge, Data)
		.bForcePickAssets(bForceAssetPicker)
		.OnMergeResolved(MereResolutionCallback);
}

class FMerge : public IMerge 
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	virtual TSharedPtr<SDockTab> GenerateMergeWidget(const UBlueprint& Object, TSharedRef<FBlueprintEditor> Editor) override;
	virtual TSharedPtr<SDockTab> GenerateMergeWidget(const UBlueprint* BaseAsset, const UBlueprint* RemoteAsset, const UBlueprint* LocalAsset, const FOnMergeResolved& MergeResolutionCallback, TSharedRef<FBlueprintEditor> Editor) override;
	virtual bool PendingMerge(const UBlueprint& BlueprintObj) const override;
	//virtual FOnMergeResolved& OnMergeResolved() const override;

	// Simplest to only allow one merge operation at a time, we could easily make this a map of Blueprint=>MergeTab
	// but doing so will complicate the tab management
	TWeakPtr<SDockTab> ActiveTab;
};
IMPLEMENT_MODULE( FMerge, Merge )

void FMerge::StartupModule()
{
	// This code will execute after your module is loaded into memory (but after global variables are initialized, of course.)


	// Registering a nomad spawner that spawns an empty dock tab on purpose - allows us to call TryInvokeTab() using our TabId later and set the content. (see GenerateMergeWidget())
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(MergeToolTabId, FOnSpawnTab::CreateStatic([] (const FSpawnTabArgs&) { return SNew(SDockTab); }))
		.SetDisplayName(NSLOCTEXT("MergeTool", "TabTitle", "Merge Tool"))
		.SetTooltipText(NSLOCTEXT("MergeTool", "TooltipText", "Used to display several versions of a blueprint that need to be merged into a single version."))
		.SetAutoGenerateMenuEntry(false);
}

void FMerge::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.
	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(MergeToolTabId);
}

TSharedPtr<SDockTab> FMerge::GenerateMergeWidget(const UBlueprint& Object, TSharedRef<FBlueprintEditor> Editor)
{
	TSharedPtr<SDockTab> ActiveTabPtr = ActiveTab.Pin();
	if( ActiveTabPtr.IsValid() )
	{
		// just bring the tab to the foreground:
		TSharedPtr<SDockTab> CurrentTab = FGlobalTabmanager::Get()->TryInvokeTab(MergeToolTabId);
		if (CurrentTab.IsValid())
		{
			check(CurrentTab == ActiveTabPtr);
			return ActiveTabPtr;
		}
		else
		{
			return nullptr;
		}
	}

	// merge the local asset with the depot, SCC provides us with the last common revision as
	// a basis for the merge:

	TSharedPtr<SWidget> Contents;

	if (!PendingMerge(Object))
	{
		// this should load up the merge-tool, with an asset picker, where they
		// can pick the asset/revisions to merge against
		Contents = GenerateMergeTabContents(Editor, nullptr, FRevisionInfo::InvalidRevision(), nullptr, FRevisionInfo::InvalidRevision(), &Object, FOnMergeResolved());
	}
	else
	{
		// @todo DO: this will probably need to be async.. pulling down some old versions of assets:
		const FString& PackageName = Object.GetOutermost()->GetName();
		const FString& AssetName = Object.GetName();

		FSourceControlStatePtr SourceControlState = FMergeToolUtils::GetSourceControlState(PackageName);
		if (!SourceControlState.IsValid())
		{
			DisplayErrorMessage(
				FText::Format(
					LOCTEXT("MergeFailedNoSourceControl", "Aborted Load of {0} from {1} because the revision control state was invalidated")
					, FText::FromString(AssetName)
					, FText::FromString(PackageName)
				)
			);

			Contents = SNew(SHorizontalBox);
		}
		else
		{
			ISourceControlState const& SourceControlStateRef = *SourceControlState;

			FRevisionInfo CurrentRevInfo = FRevisionInfo::InvalidRevision();
			const UBlueprint* RemoteBlueprint = Cast< UBlueprint >(LoadHeadRev(PackageName, AssetName, SourceControlStateRef, CurrentRevInfo));
			FRevisionInfo BaseRevInfo = FRevisionInfo::InvalidRevision();
			const UBlueprint* BaseBlueprint = Cast< UBlueprint >(LoadBaseRev(PackageName, AssetName, SourceControlStateRef, BaseRevInfo));

			Contents = GenerateMergeTabContents(Editor, BaseBlueprint, BaseRevInfo, RemoteBlueprint, CurrentRevInfo, &Object, FOnMergeResolved());
		}
	}

	TSharedPtr<SDockTab> Tab = FGlobalTabmanager::Get()->TryInvokeTab(MergeToolTabId);
	if (Tab.IsValid())
	{
		Tab->SetContent(Contents.ToSharedRef());
		ActiveTab = Tab;
	}

	return Tab;
}

TSharedPtr<SDockTab> FMerge::GenerateMergeWidget(const UBlueprint* BaseBlueprint, const UBlueprint* RemoteBlueprint, const UBlueprint* LocalBlueprint, const FOnMergeResolved& MergeResolutionCallback, TSharedRef<FBlueprintEditor> Editor)
{
	if (ActiveTab.IsValid())
	{
		TSharedPtr<SDockTab> ActiveTabPtr = ActiveTab.Pin();
		// just bring the tab to the foreground:
		TSharedPtr<SDockTab> CurrentTab = FGlobalTabmanager::Get()->TryInvokeTab(MergeToolTabId);
		if (CurrentTab.IsValid())
		{
			check(CurrentTab == ActiveTabPtr);
			return ActiveTabPtr;
		}
		else
		{
			return nullptr;
		}
	}

	// @TODO: pipe revision info through
	TSharedPtr<SWidget> TabContents = GenerateMergeTabContents(Editor, BaseBlueprint, FRevisionInfo::InvalidRevision(), RemoteBlueprint, FRevisionInfo::InvalidRevision(), LocalBlueprint, MergeResolutionCallback);

	TSharedPtr<SDockTab> Tab = FGlobalTabmanager::Get()->TryInvokeTab(MergeToolTabId);
	if (Tab.IsValid())
	{
		Tab->SetContent(TabContents.ToSharedRef());
		ActiveTab = Tab;
	}

	return Tab;
}

bool FMerge::PendingMerge(const UBlueprint& BlueprintObj) const
{
	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();

	bool bPendingMerge = false;
	if( SourceControlProvider.IsEnabled() )
	{
		FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(BlueprintObj.GetOutermost(), EStateCacheUsage::Use);
		bPendingMerge = SourceControlState.IsValid() && SourceControlState->IsConflicted();
	}
	return bPendingMerge;
}

#undef LOCTEXT_NAMESPACE
