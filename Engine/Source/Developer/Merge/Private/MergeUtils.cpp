// Copyright Epic Games, Inc. All Rights Reserved.

#include "MergeUtils.h"

#include "DiffUtils.h"
#include "IAssetTypeActions.h"
#include "UObject/UObjectHash.h"
#include "UObject/Package.h"
#include "Misc/AssetRegistryInterface.h"
#include "Misc/PackageName.h"
#include "SourceControlOperations.h"
#include "ISourceControlProvider.h"
#include "ISourceControlModule.h"
#include "SourceControlHelpers.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "MergeToolUtils"

//------------------------------------------------------------------------------
FSourceControlStatePtr FMergeToolUtils::GetSourceControlState(const FString& PackageName)
{
	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	TSharedRef<FUpdateStatus, ESPMode::ThreadSafe> UpdateStatusOperation = ISourceControlOperation::Create<FUpdateStatus>();
	UpdateStatusOperation->SetUpdateHistory(true);
	SourceControlProvider.Execute(UpdateStatusOperation, SourceControlHelpers::PackageFilename(PackageName));

	FSourceControlStatePtr State = SourceControlProvider.GetState(SourceControlHelpers::PackageFilename(PackageName), EStateCacheUsage::Use);
	if (!State.IsValid() || !State->IsSourceControlled() || !FPackageName::DoesPackageExist(PackageName))
	{
		return FSourceControlStatePtr();
	}
	else
	{
		return State;
	}
}

//------------------------------------------------------------------------------
UObject const* FMergeToolUtils::LoadRevision(const FString& AssetName, const ISourceControlRevision& DesiredRevision)
{
	FSlateNotificationManager& NotificationManager = FSlateNotificationManager::Get();

	const UObject* AssetRevision = nullptr;

	// Get the head revision of this package from source control
	FString TempFileName;
	if (DesiredRevision.Get(TempFileName))
	{
		AssetRevision = LoadAssetFromPackage(TempFileName, AssetName);
	}
	else
	{
		NotificationManager.AddNotification(
			FText::Format(
				LOCTEXT("MergedFailedToFindRevision", "Aborted Load of {0} because we could not get the requested revision")
				, FText::FromString(TempFileName)
			)
		);
	}

	return AssetRevision;
}

//------------------------------------------------------------------------------
UObject const* FMergeToolUtils::LoadRevision(const FString& PackageName, const FRevisionInfo& DesiredRevision)
{
	const UObject* AssetRevision = nullptr;
	if (UPackage* AssetPackage = LoadPackage(/*Outer =*/nullptr, *PackageName, LOAD_None))
	{
		if (UObject* Asset = AssetPackage->FindAssetInPackage())
		{
			AssetRevision = LoadRevision(Asset, DesiredRevision);
		}
	}
	return AssetRevision;
}

//------------------------------------------------------------------------------
UObject const* FMergeToolUtils::LoadRevision(const UObject* AssetObject, const FRevisionInfo& DesiredRevision)
{
	check(AssetObject->IsAsset());
	
	const UObject* AssetRevision = nullptr;
	if (DesiredRevision.Revision.IsEmpty())
	{
		// an invalid revision number represents the local copy
		AssetRevision = AssetObject;
	}
	else
	{
		FString const PackageName = AssetObject->GetOutermost()->GetName();

		FSourceControlStatePtr SourceControlState = GetSourceControlState(PackageName);
		if (SourceControlState.IsValid())
		{
			TSharedPtr<ISourceControlRevision, ESPMode::ThreadSafe> Revision = SourceControlState->FindHistoryRevision(DesiredRevision.Revision);
			if (Revision.IsValid())
			{
				FString const AssetName = AssetObject->GetName();
				AssetRevision = LoadRevision(AssetName, *Revision);
			}
		}
	}

	return AssetRevision;
}

//------------------------------------------------------------------------------
UObject const* FMergeToolUtils::LoadAssetFromPackage(const FString& PackageFileName, const FString& AssetName)
{
	FSlateNotificationManager& NotificationManager = FSlateNotificationManager::Get();
	
	const UObject* Object = nullptr;

	// Try and load that package
	const FPackagePath TempPackagePath = FPackagePath::FromLocalPath(PackageFileName);
	// TODO: @jordan.hoffmann set InOriginalPackagePath parameter if this is used for Actors
	if (UPackage* TempPackage = DiffUtils::LoadPackageForDiff(TempPackagePath, {}))
	{
		// Grab the old asset from that old package
		UObject* FoundObject = FindObject<UObject>(TempPackage, *AssetName);
		if (FoundObject)
		{
			Object = FoundObject;
		}
		else
		{
			NotificationManager.AddNotification(
				FText::Format(
					LOCTEXT("MergedFailedToFindObject", "Aborted Load of {0} because we could not find an object named {1}")
					, FText::FromString(PackageFileName)
					, FText::FromString(AssetName)
				)
			);
		}
	}
	else
	{
		NotificationManager.AddNotification(
			FText::Format(
				LOCTEXT("MergedFailedToLoadPackage", "Aborted Load of {0} because we could not load the package")
				, FText::FromString(PackageFileName)
			)
		);
	}

	return Object;
}

#undef LOCTEXT_NAMESPACE
