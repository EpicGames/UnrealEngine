// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "ISourceControlState.h"

class ISourceControlRevision;
struct FRevisionInfo;

/**  */
namespace EMergeAssetId
{
	enum Type
	{
		MergeRemote = 0,
		MergeBase,
		MergeLocal,
		Count
	};
}

/** Unified helper lib, serving shared utility functions across the merge module. */
struct FMergeToolUtils
{
	/** 
	 * Retrieves the source-control state of a specified package.
	 *
	 * @param  PackageName  The name of the asset package that you want state for.
	 * @return An invalid source-control pointer if the package doesn't exist, 
	 *         or if it doesn't have any source-control state; otherwise, a valid source-control state object.
	 */
	static FSourceControlStatePtr GetSourceControlState(const FString& PackageName);

	/** 
	 * Attempts to load the specified asset from the supplied source-control 
	 * revision.
	 *
	 * @param  AssetName		The name of the asset you want loaded from the revision.
	 * @param  DesiredRevision	The source-control revision you want loaded.
	 * @return The asset object you wanted loaded, could be null if the asset wasn't found in the supplied revision.
	 */
	static UObject const* LoadRevision(const FString& AssetName, const ISourceControlRevision& DesiredRevision);

	/** 
	 * Attempts to load the supplied revision of the specified package.
	 *
	 * @param  PackageName		The name of the package you want loaded.
	 * @param  DesiredRevision	Details the revision you want loaded.
	 * @return The package's asset object, could be null if the specified revision could not be loaded.
	 */
	static UObject const* LoadRevision(const FString& PackageName, const FRevisionInfo& DesiredRevision);

	/** 
	 * Attempts to load the specified revision of the supplied asset.
	 *
	 * @param  AssetObject		The asset you want a different revision of.
	 * @param  DesiredRevision	Details the revision you want loaded.
	 * @return The requested version of the specified asset object, could be null if the supplied revision failed to load.
	 */
	static UObject const* LoadRevision(const UObject* AssetObject, const FRevisionInfo& DesiredRevision);

	/**
	 * Attempts to load the specified asset from the provided package.
	 * 
	 * @param  PackageFileName  The file name of the package where to look for the asset.
	 * @param  AssetName        The name of the asset you want loaded from the package.
	 * @return The asset object you wanted loaded, could be null if the asset wasn't found in the package.
	 */
	static UObject const* LoadAssetFromPackage(const FString& PackageFileName, const FString& AssetName);
};
