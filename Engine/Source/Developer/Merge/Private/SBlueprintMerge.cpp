// Copyright Epic Games, Inc. All Rights Reserved.

#include "SBlueprintMerge.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SOverlay.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Engine/Blueprint.h"
#include "Misc/PackageName.h"
#include "Editor.h"
#include "Misc/MessageDialog.h"
#include "HAL/FileManager.h"
#include "Styling/AppStyle.h"
#include "ISourceControlOperation.h"
#include "SourceControlOperations.h"
#include "ISourceControlModule.h"
#include "SourceControlHelpers.h"
#include "FileHelpers.h"

#include "Kismet2/KismetEditorUtilities.h"
#include "BlueprintEditorModes.h"
#include "SMergeDetailsView.h"
#include "SMergeGraphView.h"
#include "SMergeTreeView.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "SMergeAssetPickerView.h"
#include "Subsystems/AssetEditorSubsystem.h"

#define LOCTEXT_NAMESPACE "SBlueprintMerge"

SBlueprintMerge::SBlueprintMerge()
	: Data()
{
}

static FString WriteBackup(UPackage& Package, const FString& Directory, const FString& Filename)
{
	if (GEditor)
	{
		const FString DestinationFilename = Directory + FString("/") + Filename;
		FString OriginalFilename;
		if (FPackageName::DoesPackageExist(Package.GetName(), &OriginalFilename))
		{
			if (IFileManager::Get().Copy(*DestinationFilename, *OriginalFilename) == COPY_OK)
			{
				return DestinationFilename;
			}
		}
	}
	return FString();
}

static EAppReturnType::Type PromptUserIfUndoBufferToBeCleared(UBlueprint* MergeTarget)
{
	EAppReturnType::Type OkToMerge = EAppReturnType::Yes;
	if (FKismetEditorUtilities::IsReferencedByUndoBuffer(MergeTarget))
	{
		FText TargetName = FText::FromName(MergeTarget->GetFName());

		const FText WarnMessage = FText::Format(LOCTEXT("WarnOfUndoClear", "{0} has undo actions associated with it. The undo buffer must be cleared to complete this merge. \n\nYou will not be able to undo previous actions after this. Would you like to continue?"), TargetName);
		OkToMerge = FMessageDialog::Open(EAppMsgType::YesNo, WarnMessage);
	}
	return OkToMerge;
}

void SBlueprintMerge::Construct(const FArguments InArgs, const FBlueprintMergeData& InData)
{
	check( InData.OwningEditor.Pin().IsValid() );

	// reset state
	Data = InData;
	bIsPickingAssets = InArgs._bForcePickAssets;
	OnMergeResolved  = InArgs._OnMergeResolved;
	
	if (InData.BlueprintRemote != nullptr)
	{
		RemotePath = InData.BlueprintRemote->GetOutermost()->GetName();
	}
	if (InData.BlueprintBase != nullptr)
	{
		BasePath = InData.BlueprintBase->GetOutermost()->GetName();
	}
	if (InData.BlueprintLocal != nullptr)
	{
		LocalPath = InData.BlueprintLocal->GetOutermost()->GetName();
	}
	BackupSubDir = FPaths::ProjectSavedDir() / TEXT("Backup") / TEXT("Resolve_Backup[") + FDateTime::Now().ToString(TEXT("%Y-%m-%d-%H-%M-%S")) + TEXT("]");

	FToolBarBuilder ToolbarBuilder(TSharedPtr< FUICommandList >(), FMultiBoxCustomization::None);
	ToolbarBuilder.AddToolBarButton( 
		FUIAction( FExecuteAction::CreateSP(this, &SBlueprintMerge::PrevDiff), FCanExecuteAction::CreateSP(this, &SBlueprintMerge::HasPrevDiff) )
		, NAME_None
		, LOCTEXT("PrevMergeLabel", "Prev")
		, LOCTEXT("PrevMergeTooltip", "Go to previous difference")
		, FSlateIcon(FAppStyle::GetAppStyleSetName(), "BlueprintMerge.PrevDiff")
	);
	ToolbarBuilder.AddToolBarButton(
		FUIAction(FExecuteAction::CreateSP(this, &SBlueprintMerge::NextDiff), FCanExecuteAction::CreateSP(this, &SBlueprintMerge::HasNextDiff))
		, NAME_None
		, LOCTEXT("NextMergeLabel", "Next")
		, LOCTEXT("NextMergeTooltip", "Go to next difference")
		, FSlateIcon(FAppStyle::GetAppStyleSetName(), "BlueprintMerge.NextDiff") 
	);

	auto IsInPassiveMode = [this]()
	{
		return !IsActivelyMerging();
	};

	// conflict navigation buttons:
	ToolbarBuilder.AddSeparator();
	ToolbarBuilder.AddToolBarButton( 
		FUIAction( FExecuteAction::CreateSP(this, &SBlueprintMerge::PrevConflict), FCanExecuteAction::CreateSP(this, &SBlueprintMerge::HasPrevConflict) )
		, NAME_None
		, LOCTEXT("PrevConflictLabel", "Prev Conflict")
		, LOCTEXT("PrevConflictTooltip", "Go to previous conflict")
		, FSlateIcon(FAppStyle::GetAppStyleSetName(), "BlueprintMerge.PrevDiff")
	);
	ToolbarBuilder.AddToolBarButton(
		FUIAction(FExecuteAction::CreateSP(this, &SBlueprintMerge::NextConflict), FCanExecuteAction::CreateSP(this, &SBlueprintMerge::HasNextConflict))
		, NAME_None
		, LOCTEXT("NextConflictLabel", "Next Conflict")
		, LOCTEXT("NextConflictTooltip", "Go to next conflict")
		, FSlateIcon(FAppStyle::GetAppStyleSetName(), "BlueprintMerge.NextDiff") 
	);

	// buttons for finishing the merge:
	ToolbarBuilder.AddSeparator(); 
	ToolbarBuilder.AddToolBarButton(
		FUIAction(
			FExecuteAction::CreateSP(this, &SBlueprintMerge::OnStartMerge),
			FCanExecuteAction::CreateSP(this, &SBlueprintMerge::CanStartMerge),
			FIsActionChecked(),
			FIsActionButtonVisible::CreateLambda(IsInPassiveMode))
		, NAME_None
		, LOCTEXT("StartMergeLabel", "Start Merge")
		, LOCTEXT("StartMergeTooltip", "Loads the selected blueprints and switches to an active merge (using your selections for the remote/base/local)")
		, FSlateIcon(FAppStyle::GetAppStyleSetName(), "BlueprintMerge.StartMerge")
	);
	ToolbarBuilder.AddToolBarButton(
		FUIAction(
			FExecuteAction::CreateSP(this, &SBlueprintMerge::OnAcceptRemote),
			FCanExecuteAction(), FIsActionChecked(),
			FIsActionButtonVisible::CreateSP(this, &SBlueprintMerge::IsActivelyMerging))
		, NAME_None
		, LOCTEXT("AcceptRemoteLabel", "Accept Source")
		, LOCTEXT("AcceptRemoteTooltip", "Complete the merge operation - Replaces the Blueprint with a copy of the remote file.")
		, FSlateIcon(FAppStyle::GetAppStyleSetName(), "BlueprintMerge.AcceptSource")
	);
	ToolbarBuilder.AddToolBarButton(
		FUIAction(
			FExecuteAction::CreateSP(this, &SBlueprintMerge::OnAcceptLocal),
			FCanExecuteAction(), FIsActionChecked(),
			FIsActionButtonVisible::CreateSP(this, &SBlueprintMerge::IsActivelyMerging))
		, NAME_None
		, LOCTEXT("AcceptLocalLabel", "Accept Target")
		, LOCTEXT("AcceptLocalTooltip", "Complete the merge operation - Leaves the target Blueprint unchanged.")
		, FSlateIcon(FAppStyle::GetAppStyleSetName(), "BlueprintMerge.AcceptTarget")
	);
	ToolbarBuilder.AddToolBarButton(
		FUIAction(
			FExecuteAction::CreateSP(this, &SBlueprintMerge::OnFinishMerge), 
			FCanExecuteAction(), FIsActionChecked(),
			FIsActionButtonVisible::CreateSP(this, &SBlueprintMerge::IsActivelyMerging))
		, NAME_None
		, LOCTEXT("FinishMergeLabel", "Finish Merge")
		, LOCTEXT("FinishMergeTooltip", "Complete the merge operation - saves the blueprint and resolves the conflict with the SCC provider")
		, FSlateIcon(FAppStyle::GetAppStyleSetName(), "BlueprintMerge.Finish")
	);
	ToolbarBuilder.AddToolBarButton(
		FUIAction(FExecuteAction::CreateSP(this, &SBlueprintMerge::OnCancelClicked))
		, NAME_None
		, LOCTEXT("CancelMergeLabel", "Cancel")
		, LOCTEXT("CancelMergeTooltip", "Abort the merge operation")
		, FSlateIcon(FAppStyle::GetAppStyleSetName(), "BlueprintMerge.Cancel")
	);

	TSharedRef<SWidget> Overlay = SNew(SHorizontalBox);
	if( IsActivelyMerging() )
	{
		const auto TextBlock = []( FText Text ) -> TSharedRef<SWidget>
		{
			return SNew(STextBlock)
				.Visibility(EVisibility::HitTestInvisible)
				.TextStyle(FAppStyle::Get(), "GraphPreview.CornerText")
				.Text(Text);
		};

		Overlay = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		[
			TextBlock(DiffViewUtils::GetPanelLabel(InData.BlueprintRemote, InData.RevisionRemote, LOCTEXT("RemoteLabel", "SOURCE (REMOTE)")))
		]
		+ SHorizontalBox::Slot()
		[
			TextBlock(DiffViewUtils::GetPanelLabel(InData.BlueprintBase, InData.RevisionBase, LOCTEXT("BaseLabel", "BASE")))
		]
		+ SHorizontalBox::Slot()
		[
			TextBlock(DiffViewUtils::GetPanelLabel(InData.BlueprintLocal, InData.RevisionLocal, LOCTEXT("LocalLabel", "TARGET (LOCAL)")))
		];
	}

	ChildSlot 
	[
		SNew( SVerticalBox )
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew( SHorizontalBox )
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.f)
			[
				ToolbarBuilder.MakeWidget()
			]
		]
		+ SVerticalBox::Slot()
		[
			SNew(SSplitter)
			+ SSplitter::Slot()
			.Value(.2f)
			[
				SAssignNew(TreeViewContainer, SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			]
			+ SSplitter::Slot()
			.Value(.8f)
			[
				SNew(SOverlay)
				+SOverlay::Slot()
				[
					SAssignNew(MainView, SBox)
				]
				+ SOverlay::Slot()
				.VAlign(VAlign_Bottom)
				[
					Overlay
				]
			]
		]
	];

	if (IsActivelyMerging())
	{
		OnStartMerge();
	}
	else
	{
		bIsPickingAssets = true;

		auto AssetPickerView = SNew(SMergeAssetPickerView, InData).OnAssetChanged(this, &SBlueprintMerge::OnMergeAssetSelected);
		AssetPickerControl = AssetPickerView;
	}

	Data.OwningEditor.Pin()->OnModeSet().AddSP( this, &SBlueprintMerge::OnModeChanged );
	OnModeChanged( Data.OwningEditor.Pin()->GetCurrentMode() );
}

UBlueprint* SBlueprintMerge::GetTargetBlueprint()
{
	return Data.OwningEditor.IsValid() ? Data.OwningEditor.Pin()->GetBlueprintObj() : nullptr;
}

void SBlueprintMerge::NextDiff()
{
	DiffTreeView::HighlightNextDifference(DifferencesTreeView.ToSharedRef(), RealDifferences, PrimaryDifferencesList);
}

void SBlueprintMerge::PrevDiff()
{
	 DiffTreeView::HighlightPrevDifference( DifferencesTreeView.ToSharedRef(), RealDifferences, PrimaryDifferencesList );
}

bool SBlueprintMerge::HasNextDiff() const
{
	return DifferencesTreeView.IsValid() && DiffTreeView::HasNextDifference( DifferencesTreeView.ToSharedRef(), RealDifferences );
}

bool SBlueprintMerge::HasPrevDiff() const
{
	return DifferencesTreeView.IsValid() && DiffTreeView::HasPrevDifference(DifferencesTreeView.ToSharedRef(), RealDifferences );
}

void SBlueprintMerge::NextConflict()
{
	DiffTreeView::HighlightNextDifference(DifferencesTreeView.ToSharedRef(), MergeConflicts, PrimaryDifferencesList);
}

void SBlueprintMerge::PrevConflict()
{
	DiffTreeView::HighlightPrevDifference(DifferencesTreeView.ToSharedRef(), MergeConflicts, PrimaryDifferencesList);
}

bool SBlueprintMerge::HasNextConflict() const
{
	return DifferencesTreeView.IsValid() && DiffTreeView::HasNextDifference(DifferencesTreeView.ToSharedRef(), MergeConflicts);
}

bool SBlueprintMerge::HasPrevConflict() const
{
	return DifferencesTreeView.IsValid() && DiffTreeView::HasPrevDifference(DifferencesTreeView.ToSharedRef(), MergeConflicts);
}

void SBlueprintMerge::OnStartMerge()
{
	DifferencesTreeView = DiffTreeView::CreateTreeView(&PrimaryDifferencesList);
	TreeViewContainer->SetContent(DifferencesTreeView.ToSharedRef());

	if (Data.BlueprintRemote == nullptr)
	{
		Data.BlueprintRemote = Cast<UBlueprint>(FMergeToolUtils::LoadRevision(RemotePath, Data.RevisionRemote));
	}
	if (Data.BlueprintBase == nullptr)
	{
		Data.BlueprintBase = Cast<UBlueprint>(FMergeToolUtils::LoadRevision(BasePath, Data.RevisionBase));
	}
	if (Data.BlueprintLocal == nullptr)
	{
		Data.BlueprintLocal = Cast<UBlueprint>(FMergeToolUtils::LoadRevision(LocalPath, Data.RevisionLocal));
	}
	LocalBackupPath.Empty();

	if ((Data.BlueprintRemote != nullptr) && (Data.BlueprintBase != nullptr) && (Data.BlueprintLocal != nullptr))
	{
		// Because merge operations are so destructive and can be confusing, lets write backups of the files involved:
		WriteBackup(*Data.BlueprintRemote->GetOutermost(), BackupSubDir, TEXT("RemoteAsset") + FPackageName::GetAssetPackageExtension());
		WriteBackup(*Data.BlueprintBase->GetOutermost(), BackupSubDir, TEXT("CommonBaseAsset") + FPackageName::GetAssetPackageExtension());
		LocalBackupPath = WriteBackup(*Data.BlueprintLocal->GetOutermost(), BackupSubDir, TEXT("LocalAsset") + FPackageName::GetAssetPackageExtension());

		// we want to make a read only copy of the local blueprint, since we allow 
		// the user to modify/mutate the merge result (which starts as the local copy)
		if (Data.BlueprintLocal == GetTargetBlueprint())
		{
			UBlueprint* LocalBlueprint = GetTargetBlueprint();
			// we use this to suppress blueprint compilation during StaticDuplicateObject()
			FGuardValue_Bitfield(LocalBlueprint->bDuplicatingReadOnly, true);

			UPackage* TransientPackage = GetTransientPackage();
			Data.BlueprintLocal = Cast<const UBlueprint>(StaticDuplicateObject(LocalBlueprint, TransientPackage,
				MakeUniqueObjectName(TransientPackage, LocalBlueprint->GetClass(), LocalBlueprint->GetFName())));
		}
		else
		{
			// since we didn't need to make a copy of the local-blueprint, then
			// we're most likely using a different asset for "local"; in that 
			// case, we don't want to use a copy of the file in OnAcceptLocal()
			// (the name of the blueprint would be all off, etc.)... instead,
			// we want to just duplicate the Blueprint pointer (like we do for
			// OnAcceptRemote), so we leverage the emptiness of LocalBackupPath
			// as a sentinel value (to determine which method we choose)
			LocalBackupPath.Empty();
		}

		const auto DetailsSelected = [](TWeakPtr<FBlueprintEditor> Editor)
		{
			Editor.Pin()->SetCurrentMode(FBlueprintEditorApplicationModes::BlueprintDefaultsMode);
		};
		auto DetailsView = SNew(SMergeDetailsView
			, Data
			, FOnMergeNodeSelected::CreateStatic(DetailsSelected, Data.OwningEditor)
			, PrimaryDifferencesList
			, RealDifferences
			, MergeConflicts);
		DetailsControl = DetailsView;

		const auto ComponentSelected = [](TWeakPtr<FBlueprintEditor> Editor)
		{
			Editor.Pin()->SetCurrentMode(FBlueprintEditorApplicationModes::BlueprintComponentsMode);
		};
		auto TreeView = SNew(SMergeTreeView
			, Data
			, FOnMergeNodeSelected::CreateStatic(ComponentSelected, Data.OwningEditor)
			, PrimaryDifferencesList
			, RealDifferences
			, MergeConflicts);
		TreeControl = TreeView;

		const auto GraphSelected = [](TWeakPtr<FBlueprintEditor> Editor)
		{
			Editor.Pin()->SetCurrentMode(FBlueprintEditorApplicationModes::StandardBlueprintEditorMode);
		};
		auto GraphView = SNew(SMergeGraphView
							, Data
							, FOnMergeNodeSelected::CreateStatic(GraphSelected, Data.OwningEditor)
							, PrimaryDifferencesList
							, RealDifferences
							, MergeConflicts);
		GraphControl = GraphView;

		bIsPickingAssets = false;
		OnModeChanged(Data.OwningEditor.Pin()->GetCurrentMode());
	}
	else
	{
		const FText ErrorMessage = LOCTEXT("FailedMergeLoad", "Failed to load asset(s) for merge.");
		FSlateNotificationManager::Get().AddNotification(FNotificationInfo(ErrorMessage));
	}

}

void SBlueprintMerge::OnFinishMerge()
{
	ResolveMerge(GetTargetBlueprint());
}

void SBlueprintMerge::OnCancelClicked()
{
	// should come before CloseMergeTool(), because CloseMergeTool() makes its
	// own call to OnMergeResolved (with an "Unknown" state).
	OnMergeResolved.ExecuteIfBound(GetTargetBlueprint()->GetOutermost(), EMergeResult::Cancelled);

	// if we're using the merge command-line, it might close everything once
	// a resolution is found (so the editor may be invalid now)
	if (Data.OwningEditor.IsValid())
	{
		Data.OwningEditor.Pin()->CloseMergeTool();
	}
}

void SBlueprintMerge::OnModeChanged(FName NewMode)
{
	if (!IsActivelyMerging())
	{
		MainView->SetContent(AssetPickerControl.ToSharedRef());
	}
	else if (NewMode == FBlueprintEditorApplicationModes::StandardBlueprintEditorMode ||
		NewMode == FBlueprintEditorApplicationModes::BlueprintMacroMode)
	{
		MainView->SetContent( GraphControl.ToSharedRef() );
	}
	else if (NewMode == FBlueprintEditorApplicationModes::BlueprintComponentsMode)
	{
		MainView->SetContent(TreeControl.ToSharedRef());
	}
	else if (NewMode == FBlueprintEditorApplicationModes::BlueprintDefaultsMode ||
			NewMode == FBlueprintEditorApplicationModes::BlueprintInterfaceMode)
	{
		MainView->SetContent(DetailsControl.ToSharedRef());
	}
	else
	{
		ensureMsgf(false, TEXT("Diff panel does not support mode %s"), *NewMode.ToString());
	}
}

void SBlueprintMerge::OnAcceptRemote()
{
	UBlueprint* TargetBlueprint = GetTargetBlueprint();
	if (PromptUserIfUndoBufferToBeCleared(TargetBlueprint) == EAppReturnType::Yes)
	{
		const UBlueprint* RemoteBlueprint = Data.BlueprintRemote;
		if (UBlueprint* NewBlueprint = FKismetEditorUtilities::ReplaceBlueprint(TargetBlueprint, RemoteBlueprint))
		{
			ResolveMerge(NewBlueprint);
		}
	}
}

void SBlueprintMerge::OnAcceptLocal()
{
	UBlueprint* NewBlueprint = nullptr;

	UBlueprint* TargetBlueprint = GetTargetBlueprint();
	if (PromptUserIfUndoBufferToBeCleared(TargetBlueprint) == EAppReturnType::Yes)
	{
		// we use the emptiness of LocalBackupPath as a sentinel value (to 
		// determine how we should replace the TargetBlueprint)... if 
		// Data.BlueprintLocal is malformed (and not ok to copy), then 
		// LocalBackupPath should be valid (and we need to replace with
		// the backup instead)
		if (LocalBackupPath.IsEmpty())
		{
			NewBlueprint = FKismetEditorUtilities::ReplaceBlueprint(TargetBlueprint, Data.BlueprintLocal);
		}
		else
		{
			const FString BlueprintName = TargetBlueprint->GetName();
			const UBlueprint* LocalBackupBlueprint = Cast<const UBlueprint>(FMergeToolUtils::LoadAssetFromPackage(LocalBackupPath, BlueprintName));

			NewBlueprint = FKismetEditorUtilities::ReplaceBlueprint(TargetBlueprint, LocalBackupBlueprint);
		}
	}

	if (NewBlueprint != nullptr)
	{
		ResolveMerge(NewBlueprint);
	}
}

void SBlueprintMerge::ResolveMerge(UBlueprint* ResultantBlueprint)
{
	UPackage* Package = ResultantBlueprint->GetOutermost();
	
	UE_LOGF(LogUObjectGlobals, Log, "Merging Blueprint: %ls", *Package->GetName());

	TArray<UPackage*> PackagesToSave;
	PackagesToSave.Add(Package);

	// Perform the resolve with the SCC plugin, we do this first so that the editor doesn't warn about writing to a file that is unresolved:
	ISourceControlModule::Get().GetProvider().Execute(ISourceControlOperation::Create<FResolve>(), SourceControlHelpers::PackageFilenames(PackagesToSave), EConcurrency::Synchronous);

	FEditorFileUtils::EPromptReturnCode const SaveResult = FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, /*bCheckDirty=*/ false, /*bPromptToSave=*/ false);
	if (SaveResult != FEditorFileUtils::PR_Success)
	{
		const FText ErrorMessage = FText::Format(LOCTEXT("MergeWriteFailedError", "Failed to write merged files, please look for backups in {0}"), FText::FromString(BackupSubDir));

		FNotificationInfo ErrorNotification(ErrorMessage);
		FSlateNotificationManager::Get().AddNotification(ErrorNotification);
	}

	// in the case where we have replaced/reloaded the blueprint 
	// ("TargetBlueprint" is most likely, and should be, forcefully closing in 
	// this scenario... we need to open the result to take its place)
	UBlueprint* TargetBlueprint = GetTargetBlueprint();
	if (ResultantBlueprint != TargetBlueprint)
	{
		if (GEditor)
		{
			if (TargetBlueprint)
			{
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseAllEditorsForAsset(TargetBlueprint);
			}
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(ResultantBlueprint);
		}
	}
	
	// should come before CloseMergeTool(), because CloseMergeTool() makes its
	// own call to OnMergeResolved (with an "Unknown" state).
	OnMergeResolved.ExecuteIfBound(Package, EMergeResult::Completed);

	// if we're using the merge command-line, it might close everything once
	// a resolution is found (so the editor may be invalid now)
	if (Data.OwningEditor.IsValid())
	{
		Data.OwningEditor.Pin()->CloseMergeTool();
	}
}

bool SBlueprintMerge::IsActivelyMerging() const
{
	return !bIsPickingAssets && (Data.BlueprintRemote != nullptr) &&
		(Data.BlueprintBase != nullptr) && (Data.BlueprintLocal != nullptr);
}

bool SBlueprintMerge::CanStartMerge() const
{
	return !IsActivelyMerging() && !RemotePath.IsEmpty() && !BasePath.IsEmpty() && !LocalPath.IsEmpty();
}

void SBlueprintMerge::OnMergeAssetSelected(EMergeAssetId::Type AssetId, const FAssetRevisionInfo& AssetInfo)
{
	switch (AssetId)
	{
	case EMergeAssetId::MergeRemote:
		{
			RemotePath = AssetInfo.AssetName;
			Data.RevisionRemote  = AssetInfo.Revision;
			Data.BlueprintRemote = nullptr;
			break;
		}
	case EMergeAssetId::MergeBase:
		{
			BasePath = AssetInfo.AssetName;
			Data.RevisionBase  = AssetInfo.Revision;
			Data.BlueprintBase = nullptr;
			break;
		}
	case EMergeAssetId::MergeLocal:
		{
			LocalPath = AssetInfo.AssetName;
			Data.RevisionLocal  = AssetInfo.Revision;
			Data.BlueprintLocal = nullptr;
			break;
		}
	}
}

#undef LOCTEXT_NAMESPACE
