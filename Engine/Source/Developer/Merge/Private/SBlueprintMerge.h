// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "BlueprintMergeData.h"
#include "MergeUtils.h"
#include "Widgets/Views/STreeView.h"

#define UE_API MERGE_API

class FBlueprintDifferenceTreeEntry;
class UBlueprint;
struct FAssetRevisionInfo;
struct FDiffSingleResult;

class SBlueprintMerge : public SCompoundWidget
{
public:
	UE_API SBlueprintMerge();

	SLATE_BEGIN_ARGS(SBlueprintMerge)
		: _bForcePickAssets(false)
	{}
		SLATE_ARGUMENT(bool, bForcePickAssets)
		SLATE_EVENT(FOnMergeResolved, OnMergeResolved)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments InArgs, const FBlueprintMergeData& InData);

private:
	/** Helper functions */
	UE_API UBlueprint* GetTargetBlueprint();

	/** Event handlers */
	UE_API void NextDiff();
	UE_API void PrevDiff();
	UE_API bool HasNextDiff() const;
	UE_API bool HasPrevDiff() const;

	UE_API void NextConflict();
	UE_API void PrevConflict();
	UE_API bool HasNextConflict() const;
	UE_API bool HasPrevConflict() const;

	UE_API void OnStartMerge();
	UE_API void OnFinishMerge();
	UE_API void OnCancelClicked();
	UE_API void OnModeChanged(FName NewMode);
	UE_API void OnAcceptRemote();
	UE_API void OnAcceptLocal();
	UE_API void ResolveMerge(UBlueprint* Result);

	/** 
	 * If the user has yet to pick their remote/base/local assets, then they're
	 * not "actively" merging yet. This query checks the state of the merge-tool
	 * to see if the user has selected to "Start Merge" yet.
	 *
	 * @return False if the AssetPickerControl should be up, otherwise true.
	 */
	UE_API bool IsActivelyMerging() const;

	/** 
	 * The user cannot start a merge until they have chosen a remote, base, &
	 * local asset/revision to use in the merge. This checks the state of their 
	 * choices.
	 *
	 * @return True, if the user has selected all three merge assets.
	 */
	UE_API bool CanStartMerge() const;

	/** 
	 * Callback function, utilized every time the user picks an asset/revision  
	 * in the AssetPickerControl view. Records the user's choices, so this can 
	 * start the merge appropriately.
	 * 
	 * @param  AssetId		Defines the asset that was altered (remote, base, or local).
	 * @param  AssetInfo	Defines the asset selection (name, revision, etc.)
	 */
	UE_API void OnMergeAssetSelected(EMergeAssetId::Type AssetId, const FAssetRevisionInfo& AssetInfo);

	FBlueprintMergeData		Data; 
	FString					BackupSubDir;

	TSharedPtr<SBox>		MainView;

	/** 
	 * We track the package name-paths for the remote, base, & local assets (so
	 * we know what to load when the user starts an active merge)... Used to 
	 * determine when a merge can be started.
	 */
	FString	RemotePath;
	FString	BasePath;
	FString	LocalPath;
	/** 
	 * When we make a malformed copy (readable data only) of the local 
	 * Blueprint, then we fill this in with a backup file path; that way, the 
	 * file can be copied for OnAcceptLocal(), instead of the loaded Blueprint 
	 * object (which is "malformed").
	 */
	FString	LocalBackupPath;

	TSharedPtr<SWidget> GraphControl;
	TSharedPtr<SWidget> TreeControl;
	TSharedPtr<SWidget> DetailsControl;
	TSharedPtr<SWidget> AssetPickerControl;
	
	bool bIsPickingAssets;
	FOnMergeResolved OnMergeResolved;

	/** Container widget for the treeview of differences: */
	TSharedPtr<SBorder> TreeViewContainer;

	/** Treeview to display all differences collected across all panels: */
	TSharedPtr< STreeView< TSharedPtr< class FBlueprintDifferenceTreeEntry > > > DifferencesTreeView;

	/** List of differences collected across all panels: */
	TArray< TSharedPtr<class FBlueprintDifferenceTreeEntry> > PrimaryDifferencesList;

	/** List of all differences, cached so that we can iterate only the differences and not labels, etc: */
	TArray< TSharedPtr<class FBlueprintDifferenceTreeEntry> > RealDifferences;

	/** List of all merge conflicts: */
	TArray< TSharedPtr<class FBlueprintDifferenceTreeEntry> > MergeConflicts;

	// This has to be allocated here because SListView cannot own the list
	// that it is displaying. It also seems like the display list *has*
	// to be a list of TSharedPtrs.
	TArray< TSharedPtr<struct FDiffSingleResult> > LocalDiffResults;
	TArray< TSharedPtr<struct FDiffSingleResult> > RemoteDiffResults;
};

#undef UE_API
