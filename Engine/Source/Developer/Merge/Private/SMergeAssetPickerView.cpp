// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMergeAssetPickerView.h"
#include "Templates/SubclassOf.h"
#include "Engine/Blueprint.h"
#include "Widgets/SBoxPanel.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SSplitter.h"
#include "Styling/AppStyle.h"
#include "AssetRegistry/AssetData.h"
#include "Editor.h"
#include "BlueprintMergeData.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "SBlueprintRevisionMenu.h"
#include "Engine/Selection.h"

#define LOCTEXT_NAMESPACE "SMergeAssetPickerView"

/*******************************************************************************
 * SMergeAssetPickerPanel
 ******************************************************************************/

class SMergeAssetPickerPanel : public SCompoundWidget
{
	DECLARE_DELEGATE_OneParam(FOnAssetChanged, const UObject*);
	DECLARE_DELEGATE_OneParam(FOnRevisionChanged, const FRevisionInfo&);

public:
	SLATE_BEGIN_ARGS(SMergeAssetPickerPanel)
		: _AssetClass(UBlueprint::StaticClass())
		, _SelectedAsset(nullptr)
		, _RevisionInfo(FRevisionInfo::InvalidRevision())
	{}
		SLATE_ARGUMENT(TSubclassOf<UBlueprint>, AssetClass)
		SLATE_ARGUMENT(const UBlueprint*, SelectedAsset)
		SLATE_ARGUMENT(FRevisionInfo, RevisionInfo)

		SLATE_EVENT(FOnAssetChanged, OnAssetChanged)
		SLATE_EVENT(FOnRevisionChanged, OnRevisionChanged)
	SLATE_END_ARGS()

	/**  */
	SMergeAssetPickerPanel() 
		: SelectedAsset(nullptr)
		, SelectedRevision(FRevisionInfo::InvalidRevision())
	{}

	/** 
	 * Builds the slate UI for SMergeAssetPickerPanel (as well as caches off any
	 * important data that was supplied as part of the FArguments struct).
	 *
	 * @param  Args	 The set of slate arguments that are used to customize this panel.
	 */
	void Construct(const FArguments Args);

	/** 
	 * Prior to launch, sometimes temp assets are created for the merge-tool 
	 * (old revisions are dumped into the temp directory, etc.). This function
	 * determines if the currently selected asset (for this panel) is one of 
	 * those temporary asset files.
	 *
	 * @return True if the selected asset is a temporary one, other wise false (if it is a legit asset, or if one isn't selected).
	 */
	bool IsTempAssetSelected() const;

private:
	TSharedRef<SWidget> MakeAssetPicker();
	FText  GetAssetPickerTextValue() const;
	FReply OnUseSelectedAssetClick();
	void   ResetSelectedAsset(const UObject* NewSelection);
	bool   IsBrowseButtonEnabled() const;
	FReply OnBrowseToPickedAsset();

	TSharedRef<SWidget> MakeAssetPickerMenu();
	void OnAssetSelected(FAssetData const& AssetData);

	TSharedRef<SWidget> MakeRevisionPicker();
	FText GetRevisionTextValue() const;

	TSharedRef<SWidget> MakeRevisionPickerMenu();
	void OnRevisionSelected(FRevisionInfo const& RevisionInfo);

	TSubclassOf<UBlueprint> AssetClass;
	UBlueprint const* SelectedAsset;
	FOnAssetChanged OnAssetChanged;
	FRevisionInfo SelectedRevision;
	FOnRevisionChanged OnRevisionChanged;
	TSharedPtr<SMenuAnchor> AssetPicker;
};

//------------------------------------------------------------------------------
void SMergeAssetPickerPanel::Construct(const FArguments InArgs)
{
	AssetClass        = InArgs._AssetClass;
	SelectedAsset     = InArgs._SelectedAsset;
	SelectedRevision  = InArgs._RevisionInfo;
	OnAssetChanged    = InArgs._OnAssetChanged;
	OnRevisionChanged = InArgs._OnRevisionChanged;

	ChildSlot
	[
		SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SBox)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
					.HAlign(HAlign_Left)
					.AutoHeight()
					.Padding(0, 2)
				[
					MakeAssetPicker()
				]
				+ SVerticalBox::Slot()
					.HAlign(HAlign_Left)
					.AutoHeight()
					.Padding(0, 2)
				[
					MakeRevisionPicker()
				]
			]
		]
	];
}

//------------------------------------------------------------------------------
bool SMergeAssetPickerPanel::IsTempAssetSelected() const
{
	return (SelectedAsset != nullptr) && (!SelectedAsset->IsAsset() ||(SelectedAsset->GetOutermost() == GetTransientPackage()) || 
		SelectedAsset->GetPathName().StartsWith("/Temp/"));
}

//------------------------------------------------------------------------------
TSharedRef<SWidget> SMergeAssetPickerPanel::MakeAssetPicker()
{
	TSharedPtr<SButton> BrowseButton;
	SAssignNew(BrowseButton, SButton)
		.ButtonStyle(FAppStyle::Get(), "NoBorder")
		.OnClicked(FOnClicked::CreateSP(this, &SMergeAssetPickerPanel::OnBrowseToPickedAsset))
		.ContentPadding(0)
		.ToolTipText(NSLOCTEXT("GraphEditor", "ObjectGraphPin_Browse", "Show the selected asset in the content browser."))
	[
		SNew(SImage)
			.Image(FAppStyle::GetBrush(TEXT("Icons.Search")))
	];
	BrowseButton->SetEnabled(TAttribute<bool>(this, &SMergeAssetPickerPanel::IsBrowseButtonEnabled));

	return SNew(SHorizontalBox)
	+ SHorizontalBox::Slot()
		.Padding(2, 0)
		.MaxWidth(150.0f)
	[
		SAssignNew(AssetPicker, SComboButton)
			.ButtonStyle(FAppStyle::Get(), "PropertyEditor.AssetComboStyle")
			.ContentPadding(FMargin(2, 2, 2, 1))
			.OnGetMenuContent(this, &SMergeAssetPickerPanel::MakeAssetPickerMenu)
			.ButtonContent()
		[
			SNew(SBox).WidthOverride(150.f)
			[
				SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), "PropertyEditor.AssetClass")
					.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
					.Text(this, &SMergeAssetPickerPanel::GetAssetPickerTextValue)
			]
		]
	]
	+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(1, 0)
		.VAlign(VAlign_Center)
	[
		SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "NoBorder")
			.OnClicked(FOnClicked::CreateSP(this, &SMergeAssetPickerPanel::OnUseSelectedAssetClick))
			.ContentPadding(1.f)
			.ToolTipText(NSLOCTEXT("GraphEditor", "ObjectGraphPin_Use", "Use content browser selection."))
		[
			SNew(SImage)
				.Image(FAppStyle::GetBrush(TEXT("Icons.CircleArrowLeft")))
		]
	]
	+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(1, 0)
		.VAlign(VAlign_Center)
	[
		BrowseButton.ToSharedRef()
	];
}

//------------------------------------------------------------------------------
FText SMergeAssetPickerPanel::GetAssetPickerTextValue() const
{
	FText AssetPickerTitle = LOCTEXT("SelectAsset", "Select an Asset...");
	if (SelectedAsset != nullptr)
	{
		if (UClass* BlueprintClass = SelectedAsset->GeneratedClass)
		{
			AssetPickerTitle = BlueprintClass->GetDisplayNameText();
		}
		else
		{
			AssetPickerTitle = FText::FromString(SelectedAsset->GetName());
		}
	}
	return AssetPickerTitle;
}

//------------------------------------------------------------------------------
FReply SMergeAssetPickerPanel::OnUseSelectedAssetClick()
{
	FEditorDelegates::LoadSelectedAssetsIfNeeded.Broadcast();

	UObject* SelectedObject = GEditor->GetSelectedObjects()->GetTop(AssetClass);
	if (SelectedObject != nullptr)
	{
		ResetSelectedAsset(SelectedObject);
	}

	return FReply::Handled();
}

//------------------------------------------------------------------------------
void SMergeAssetPickerPanel::ResetSelectedAsset(const UObject* NewSelection)
{
	SelectedAsset = Cast<UBlueprint>(NewSelection);
	OnAssetChanged.ExecuteIfBound(SelectedAsset);

	// reset the revision (to current)
	SelectedRevision = FRevisionInfo::InvalidRevision();
	OnRevisionChanged.ExecuteIfBound(SelectedRevision);
}

//------------------------------------------------------------------------------
bool SMergeAssetPickerPanel::IsBrowseButtonEnabled() const
{
	return (SelectedAsset != nullptr);
}

//------------------------------------------------------------------------------
FReply SMergeAssetPickerPanel::OnBrowseToPickedAsset()
{
	if (SelectedAsset != nullptr)
	{
		TArray<UObject*> BrowseObjects;
		// @TODO: SyncBrowserToObjects() should really take an array of const objects
		BrowseObjects.Add(const_cast<UBlueprint*>(SelectedAsset));

		GEditor->SyncBrowserToObjects(BrowseObjects);
	}
	return FReply::Handled();
}

//------------------------------------------------------------------------------
TSharedRef<SWidget> SMergeAssetPickerPanel::MakeAssetPickerMenu()
{
	FAssetPickerConfig AssetPickerConfig;
	AssetPickerConfig.Filter.ClassPaths.Add(AssetClass->GetClassPathName());
	AssetPickerConfig.bAllowNullSelection = true;
	AssetPickerConfig.Filter.bRecursiveClasses = true;
	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateSP(this, &SMergeAssetPickerPanel::OnAssetSelected);
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
	AssetPickerConfig.bAllowDragging = false;

	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

	return SNew(SBox)
		.WidthOverride(300)
		.HeightOverride(300)
	[
		SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Menu.Background"))
		[
			ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
		]
	];
}

//------------------------------------------------------------------------------
void SMergeAssetPickerPanel::OnAssetSelected(FAssetData const& AssetData)
{
	AssetPicker->SetIsOpen(false);
	ResetSelectedAsset(AssetData.GetAsset());
}

//------------------------------------------------------------------------------
TSharedRef<SWidget> SMergeAssetPickerPanel::MakeRevisionPicker()
{
	return SNew(SHorizontalBox)
	+ SHorizontalBox::Slot()
		.Padding(2, 0)
		.MaxWidth(150.0f)
	[
		SAssignNew(AssetPicker, SComboButton)
			.ButtonStyle(FAppStyle::Get(), "PropertyEditor.AssetComboStyle")
			.ContentPadding(FMargin(2, 2, 2, 1))
			.OnGetMenuContent(this, &SMergeAssetPickerPanel::MakeRevisionPickerMenu)
			.ButtonContent()
		[
			SNew(SBox).WidthOverride(150.f)
			[
				SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), "PropertyEditor.AssetClass")
					.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
					.Text(this, &SMergeAssetPickerPanel::GetRevisionTextValue)
			]
		]
	];
}

//------------------------------------------------------------------------------
FText SMergeAssetPickerPanel::GetRevisionTextValue() const
{
	FText RevisionText = LOCTEXT("UnknownRevision", "Unknown Revision");
	if (SelectedAsset == nullptr)
	{
		RevisionText = LOCTEXT("PickRevision", "Pick a Revision...");
	}
	else if (!SelectedRevision.Revision.IsEmpty())
	{
		RevisionText = FText::Format(LOCTEXT("RevisionNum", "Revision {0}"), FText::FromString(SelectedRevision.Revision));
	}
	else if (!IsTempAssetSelected())
	{
		RevisionText = LOCTEXT("LocalRevision", "Local Copy");
	}
	else
	{
		RevisionText = LOCTEXT("UnknownRevision", "Unknown Revision");
	}
	return RevisionText;
}

//------------------------------------------------------------------------------
TSharedRef<SWidget> SMergeAssetPickerPanel::MakeRevisionPickerMenu()
{
	if (SelectedAsset == nullptr)
	{
		return SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Menu.Background"))
		[
			SNew(STextBlock).Text(LOCTEXT("SelectAssetFirst", "Select an asset first."))
		];
	}
	else if (!IsTempAssetSelected())
	{
		return SNew(SBlueprintRevisionMenu, SelectedAsset)
			.bIncludeLocalRevision(true)
			.OnRevisionSelected(this, &SMergeAssetPickerPanel::OnRevisionSelected);
	}
	else
	{
		FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection =*/true, /*CommandList =*/nullptr);
		MenuBuilder.BeginSection("RevisionSection", LOCTEXT("Revisions", "Revisions"));
		{
			FText const RevisionTooltip = LOCTEXT("UnknownRevisionTip", "The selected asset must be a temporary one created by the merge-tool (with this preordained revision).");
			MenuBuilder.AddMenuEntry(TAttribute<FText>(this, &SMergeAssetPickerPanel::GetRevisionTextValue),
				RevisionTooltip, FSlateIcon(), FUIAction());
		}
		MenuBuilder.EndSection();

		return SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Menu.Background"))
		[
			MenuBuilder.MakeWidget()
		];
	}
}

//------------------------------------------------------------------------------
void SMergeAssetPickerPanel::OnRevisionSelected(FRevisionInfo const& RevisionInfo)
{
	SelectedRevision = RevisionInfo;
	OnRevisionChanged.ExecuteIfBound(RevisionInfo);
}

/*******************************************************************************
 * SMergeAssetPickerView
 ******************************************************************************/

//------------------------------------------------------------------------------
void SMergeAssetPickerView::Construct(const FArguments InArgs, const FBlueprintMergeData& InData)
{
	MergeAssetSet[EMergeAssetId::MergeRemote].AssetName = InData.BlueprintRemote->GetOutermost()->GetName();
	MergeAssetSet[EMergeAssetId::MergeRemote].Revision = InData.RevisionRemote;
	MergeAssetSet[EMergeAssetId::MergeBase].AssetName = InData.BlueprintBase->GetOutermost()->GetName();
	MergeAssetSet[EMergeAssetId::MergeBase].Revision = InData.RevisionBase;
	MergeAssetSet[EMergeAssetId::MergeLocal].AssetName = InData.BlueprintLocal->GetOutermost()->GetName();
	MergeAssetSet[EMergeAssetId::MergeLocal].Revision = InData.RevisionLocal;
	
	OnAssetChanged = InArgs._OnAssetChanged;

	ChildSlot
	[
		SNew(SSplitter)
		+ SSplitter::Slot()
		[
			SNew(SMergeAssetPickerPanel)
				.SelectedAsset(InData.BlueprintRemote)
				.RevisionInfo(InData.RevisionRemote)
				.OnRevisionChanged(this, &SMergeAssetPickerView::HandleRevisionChange, EMergeAssetId::MergeRemote)
				.OnAssetChanged(this, &SMergeAssetPickerView::HandleAssetChange, EMergeAssetId::MergeRemote)
		] 
		+ SSplitter::Slot()
		[
			SNew(SMergeAssetPickerPanel)
				.SelectedAsset(InData.BlueprintBase)
				.RevisionInfo(InData.RevisionBase)
				.OnRevisionChanged(this, &SMergeAssetPickerView::HandleRevisionChange, EMergeAssetId::MergeBase)
				.OnAssetChanged(this, &SMergeAssetPickerView::HandleAssetChange, EMergeAssetId::MergeBase)
		]
		+ SSplitter::Slot()
		[
			SNew(SMergeAssetPickerPanel)
				.SelectedAsset(InData.BlueprintLocal)
				.RevisionInfo(InData.RevisionLocal)
				.OnRevisionChanged(this, &SMergeAssetPickerView::HandleRevisionChange, EMergeAssetId::MergeLocal)
				.OnAssetChanged(this, &SMergeAssetPickerView::HandleAssetChange, EMergeAssetId::MergeLocal)
		]
	];
}

//------------------------------------------------------------------------------
void SMergeAssetPickerView::HandleAssetChange(const UObject* NewAsset, EMergeAssetId::Type PanelId)
{
	if (NewAsset == nullptr)
	{
		MergeAssetSet[PanelId].AssetName.Empty();
	}
	else
	{
		MergeAssetSet[PanelId].AssetName = NewAsset->GetOutermost()->GetName();
	}
	OnAssetChanged.ExecuteIfBound(PanelId, MergeAssetSet[PanelId]);
}

//------------------------------------------------------------------------------
void SMergeAssetPickerView::HandleRevisionChange(const FRevisionInfo& NewRevision, EMergeAssetId::Type PanelId)
{
	MergeAssetSet[PanelId].Revision = NewRevision;
	OnAssetChanged.ExecuteIfBound(PanelId, MergeAssetSet[PanelId]);
}

#undef LOCTEXT_NAMESPACE
