// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "IAssetTypeActions.h"
#include "MergeUtils.h"

struct FBlueprintMergeData;

/*******************************************************************************
 * FAssetRevisionInfo
 ******************************************************************************/

struct FAssetRevisionInfo
{
	FString AssetName;
	FRevisionInfo Revision;
};

/*******************************************************************************
 * SMergeAssetPickerView
 ******************************************************************************/

class SMergeAssetPickerView : public SCompoundWidget
{
	DECLARE_DELEGATE_TwoParams(FOnMergeAssetChanged, EMergeAssetId::Type, const FAssetRevisionInfo&)

public:
	SLATE_BEGIN_ARGS(SMergeAssetPickerView){}
		SLATE_EVENT(FOnMergeAssetChanged, OnAssetChanged)
	SLATE_END_ARGS()

	void Construct(const FArguments InArgs, const FBlueprintMergeData& InData);

private:
	void HandleAssetChange(const UObject* NewAsset, EMergeAssetId::Type PanelId);
	void HandleRevisionChange(const FRevisionInfo& NewRevision, EMergeAssetId::Type PanelId);

	FOnMergeAssetChanged OnAssetChanged;
	FAssetRevisionInfo MergeAssetSet[EMergeAssetId::Count];
};
