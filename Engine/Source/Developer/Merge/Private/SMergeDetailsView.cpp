// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMergeDetailsView.h"
#include "IDetailsView.h"
#include "Widgets/Layout/SSplitter.h"


void SMergeDetailsView::Construct(const FArguments InArgs
	, const FBlueprintMergeData& InData
	, FOnMergeNodeSelected SelectionCallback
	, TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >& OutTreeEntries
	, TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >& OutRealDifferences
	, TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >& OutConflicts)
{
	Data = InData;
	CurrentMergeConflict = INDEX_NONE;

	const UObject* RemoteCDO = DiffUtils::GetCDO(InData.BlueprintRemote);
	const UObject* BaseCDO = DiffUtils::GetCDO(InData.BlueprintBase);
	const UObject* LocalCDO = DiffUtils::GetCDO(InData.BlueprintLocal);

	// EMergeParticipant::Remote
	{
		DetailsViews.Add(
			FDetailsDiff(RemoteCDO )
		);
	}
	// EMergeParticipant::Base
	{
		DetailsViews.Add(
			FDetailsDiff(BaseCDO)
		);
	}
	// EMergeParticipant::Local
	{
		DetailsViews.Add(
			FDetailsDiff(LocalCDO)
		);
	}
	
	TArray<FSingleObjectDiffEntry> RemoteDifferences;
	GetBaseDetails().DiffAgainst(GetRemoteDetails(), RemoteDifferences);
	TArray<FSingleObjectDiffEntry> LocalDifferences;
	GetBaseDetails().DiffAgainst(GetLocalDetails(), LocalDifferences);

	const auto GetPropertyNames = [](const TArray<FSingleObjectDiffEntry>& FromDifferences, TArray<FPropertySoftPath>& OutNames)
	{
		for (const auto& Entry : FromDifferences)
		{
			OutNames.Push(Entry.Identifier);
		}
	};

	TArray<FPropertySoftPath> RemoteDifferingProperties;
	GetPropertyNames(RemoteDifferences, RemoteDifferingProperties);
	TArray<FPropertySoftPath> LocalDifferingProperties;
	GetPropertyNames(LocalDifferences, LocalDifferingProperties);

	FPropertySoftPathSet RemoteDifferingPropertiesSet(RemoteDifferingProperties);
	FPropertySoftPathSet LocalDifferingPropertiesSet(LocalDifferingProperties);
	FPropertySoftPathSet BaseDifferingPropertiesSet(RemoteDifferingPropertiesSet.Union(LocalDifferingPropertiesSet));
	MergeConflicts = RemoteDifferingPropertiesSet.Intersect(LocalDifferingPropertiesSet).Array();

	TArray<FPropertySoftPath> RemoteVisibleProperties = GetRemoteDetails().GetDisplayedProperties();
	TArray<FPropertySoftPath> BaseVisibleProperties = GetBaseDetails().GetDisplayedProperties();
	TArray<FPropertySoftPath> LocalVisibleProperties = GetLocalDetails().GetDisplayedProperties();
	int IterRemote = 0;
	int IterBase = 0;
	int IterLocal = 0;

	bool bDoneRemote = IterRemote == RemoteVisibleProperties.Num();
	bool bDoneBase = IterBase == BaseVisibleProperties.Num();
	bool bDoneLocal = IterLocal == LocalVisibleProperties.Num();

	bool bAnyConflict = false;

	struct FDiffPair
	{
		FPropertySoftPath Identifier;
		FText Label;
		bool bConflicted;
	};

	const auto ProcessPotentialDifference = []( const FPropertySoftPath& PropertyIdentifier
											, TArray<FSingleObjectDiffEntry> const& InRemoteDifferences
											, TArray<FSingleObjectDiffEntry> const& InLocalDifferences
											, TArray< FDiffPair >& OutProcessedDifferences
											, bool& bOutAnyConflict )
	{
		const FText RemoteLabel = NSLOCTEXT("SMergeDetailsView", "RemoteLabel", "Remote");
		const FText LocalLabel = NSLOCTEXT("SMergeDetailsView", "LocalLabel", "Local");

		const auto FindDiffering = [](TArray< FSingleObjectDiffEntry > const& InDifferences, const FPropertySoftPath& InPropertyIdentifier) -> const FSingleObjectDiffEntry*
		{
			for (const auto& Difference : InDifferences)
			{
				if (Difference.Identifier == InPropertyIdentifier)
				{
					return &Difference;
				}
			}
			return nullptr;
		};

		const FSingleObjectDiffEntry* RemoteDiffering = FindDiffering(InRemoteDifferences, PropertyIdentifier);
		const FSingleObjectDiffEntry* LocalDiffering = FindDiffering(InLocalDifferences, PropertyIdentifier);
		if (RemoteDiffering && LocalDiffering)
		{
			// conflicting change:
			bOutAnyConflict = true;

			FText Label1 = DiffViewUtils::PropertyDiffMessage(*RemoteDiffering, RemoteLabel);
			FText Label2 = DiffViewUtils::PropertyDiffMessage(*LocalDiffering, LocalLabel);
			FText FinalLabel = FText::Format(NSLOCTEXT("SMergeDetailsView", "PropertyConflict", "Conflict: {0} and {1}"), Label1, Label2);

			FDiffPair Difference = { RemoteDiffering->Identifier, FinalLabel, true };
			OutProcessedDifferences.Push(Difference);
		}
		else if (RemoteDiffering)
		{
			FText Label = DiffViewUtils::PropertyDiffMessage(*RemoteDiffering, RemoteLabel);
			FDiffPair Difference = { RemoteDiffering->Identifier,  Label, false };
			OutProcessedDifferences.Push(Difference);
		}
		else if (LocalDiffering)
		{
			FText Label = DiffViewUtils::PropertyDiffMessage(*LocalDiffering, LocalLabel);
			FDiffPair Difference = { LocalDiffering->Identifier, Label, false };
			OutProcessedDifferences.Push(Difference);
		}
	};

	/*
		DifferingProperties is an ordered list of all differing properties (properties added, removed or changed) in remote or local.
		Strictly speaking it's impossible to guarantee that we'll traverse remote and local differences in the same order (for instance
		because property layout could somehow change between revisions) but in practice the following works:

		1. Iterate properties in base, add any properties that differ in remote or local to DifferingProperties.
		2. Iterate properties in remote, add any new properties to DifferingProperties
		3. Iterate properties in local, add any new properties to DifferingProperties, if they have not already been added
	*/
	const auto AddPropertiesOrdered = [](FPropertySoftPath const& Property, const FPropertySoftPathSet& InDifferingProperties, TArray<FPropertySoftPath>& ResultingProperties)
	{
		// contains check here is O(n), so we're needlessly n^2:
		if (!ResultingProperties.Contains(Property))
		{
			if (InDifferingProperties.Contains(Property))
			{
				ResultingProperties.Add(Property);
			}
		}
	};

	TArray< FDiffPair > OrderedDifferences;
	while( !bDoneRemote || !bDoneBase || !bDoneLocal )
	{
		bool bLocalMatchesBase = !bDoneLocal && !bDoneBase && BaseVisibleProperties[IterBase] == LocalVisibleProperties[IterLocal];
		bool bRemoteMatchesBase = !bDoneRemote && !bDoneBase && BaseVisibleProperties[IterBase] == RemoteVisibleProperties[IterRemote];

		if( (bRemoteMatchesBase && bLocalMatchesBase) ||
			(bDoneLocal && bRemoteMatchesBase ) ||
			(bDoneRemote && bLocalMatchesBase ) )
		{
			ProcessPotentialDifference(BaseVisibleProperties[IterBase], RemoteDifferences, LocalDifferences, OrderedDifferences, bAnyConflict );
			AddPropertiesOrdered(BaseVisibleProperties[IterBase], BaseDifferingPropertiesSet, DifferingProperties);

			if (!bDoneLocal)
			{
				++IterLocal;
			}
			if (!bDoneRemote)
			{
				++IterRemote;
			}
			if(!bDoneBase )
			{
				++IterBase;
			}
		}
		else if ( !bDoneRemote && !bRemoteMatchesBase )
		{
			ProcessPotentialDifference(RemoteVisibleProperties[IterRemote], RemoteDifferences, LocalDifferences, OrderedDifferences, bAnyConflict);
			AddPropertiesOrdered(RemoteVisibleProperties[IterRemote], RemoteDifferingPropertiesSet, DifferingProperties);
			++IterRemote;
		}
		else if( !bDoneLocal && !bLocalMatchesBase )
		{
			ProcessPotentialDifference(LocalVisibleProperties[IterLocal], RemoteDifferences, LocalDifferences, OrderedDifferences, bAnyConflict);
			AddPropertiesOrdered(LocalVisibleProperties[IterLocal], LocalDifferingPropertiesSet, DifferingProperties);
			++IterLocal;
		}
		else if( bDoneLocal && bDoneRemote && !bDoneBase )
		{
			++IterBase;
		}

		bDoneRemote = IterRemote == RemoteVisibleProperties.Num();
		bDoneBase = IterBase == BaseVisibleProperties.Num();
		bDoneLocal = IterLocal == LocalVisibleProperties.Num();
	}

	TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> > Children;
	if( OrderedDifferences.Num() == 0 )
	{
		Children.Push( FBlueprintDifferenceTreeEntry::NoDifferencesEntry() );
	}
	else
	{
		const auto CreateDetailsMergeWidget = [](FDiffPair Entry) -> TSharedRef<SWidget>
		{
			return SNew(STextBlock)
				.Text(Entry.Label)
				.ColorAndOpacity(Entry.bConflicted ? DiffViewUtils::Conflicting() : DiffViewUtils::Differs());
		};

		const auto FocusDetailsDifferenceEntry = []( FPropertySoftPath PropertyIdentifier, SMergeDetailsView* Parent, FOnMergeNodeSelected InSelectionCallback )
		{
			InSelectionCallback.ExecuteIfBound();
			Parent->HighlightDifference(PropertyIdentifier);
		};

		for( const auto& Difference : OrderedDifferences )
		{
			auto Entry = TSharedPtr<FBlueprintDifferenceTreeEntry>(
				new FBlueprintDifferenceTreeEntry(
					FOnDiffEntryFocused::CreateStatic(FocusDetailsDifferenceEntry, Difference.Identifier, this, SelectionCallback)
					, FGenerateDiffEntryWidget::CreateStatic(CreateDetailsMergeWidget, Difference)
					, TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >()
				)
			);
			Children.Push(Entry);
			OutRealDifferences.Push(Entry);
			if( Difference.bConflicted )
			{
				OutConflicts.Push(Entry);
			}
		}
	}

	const auto ForwardSelection = [](FOnMergeNodeSelected InSelectionCallback)
	{
		// This allows the owning control to focus the correct tab (or do whatever else it likes):
		InSelectionCallback.ExecuteIfBound();
	};

	TSharedPtr<FBlueprintDifferenceTreeEntry> Category = FBlueprintDifferenceTreeEntry::CreateCategoryEntryForMerge(
		NSLOCTEXT("FBlueprintDifferenceTreeEntry", "DefaultsLabel", "Defaults"),
		NSLOCTEXT("FBlueprintDifferenceTreeEntry", "DefaultsTooltip", "The list of changes made in the Defaults panel"),
		FOnDiffEntryFocused::CreateStatic(ForwardSelection, SelectionCallback), 
		Children, RemoteDifferences.Num() != 0, LocalDifferences.Num() != 0, bAnyConflict);
	OutTreeEntries.Push(Category);

	CurrentDifference = -1;

	ChildSlot[
		SNew(SSplitter)
		+ SSplitter::Slot()
		[
			GetRemoteDetails().DetailsWidget()
		] 
		+ SSplitter::Slot()
		[
			GetBaseDetails().DetailsWidget()
		]
		+ SSplitter::Slot()
		[
			GetLocalDetails().DetailsWidget()
		]
	];
}

void SMergeDetailsView::HighlightDifference(FPropertySoftPath Path)
{
	for (auto& DetailDiff : DetailsViews)
	{
		DetailDiff.HighlightProperty(Path);
	}
}

FDetailsDiff& SMergeDetailsView::GetRemoteDetails()
{
	return DetailsViews[EMergeParticipant::Remote];
}

FDetailsDiff& SMergeDetailsView::GetBaseDetails()
{
	return DetailsViews[EMergeParticipant::Base];
}

FDetailsDiff& SMergeDetailsView::GetLocalDetails()
{
	return DetailsViews[EMergeParticipant::Local];
}

