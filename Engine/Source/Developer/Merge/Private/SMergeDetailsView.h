// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "BlueprintMergeData.h"
#include "DiffUtils.h"
#include "DetailsDiff.h"

class SMergeDetailsView : public SCompoundWidget
{
public:
	virtual ~SMergeDetailsView() = default;

	SLATE_BEGIN_ARGS(SMergeDetailsView)
	{}
	SLATE_END_ARGS()

	void Construct(const FArguments InArgs
					, const FBlueprintMergeData& InData
					, FOnMergeNodeSelected SelectionCallback
					, TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >& OutTreeEntries
					, TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >& OutRealDifferences
					, TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >& OutConflicts);
private:
	void HighlightDifference(FPropertySoftPath Path);
	FDetailsDiff& GetRemoteDetails();
	FDetailsDiff& GetBaseDetails();
	FDetailsDiff& GetLocalDetails();

	TArray< FPropertySoftPath > MergeConflicts;
	int CurrentMergeConflict;

	FBlueprintMergeData Data;
	TArray< FDetailsDiff > DetailsViews;

	/** These have been duplicated from FCDODiffControl, opportunity to refactor exists: */
	TArray< FPropertySoftPath > DifferingProperties;
	int CurrentDifference;
};
