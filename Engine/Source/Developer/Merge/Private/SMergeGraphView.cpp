// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMergeGraphView.h"
#include "Engine/Blueprint.h"
#include "Widgets/Layout/SSplitter.h"
#include "EdGraph/EdGraph.h"
#include "Widgets/Images/SImage.h"
#include "Framework/Docking/TabManager.h"
#include "Styling/AppStyle.h"
#include "GraphDiffControl.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "SMergeGraphView"

const FName MergeMyBluerpintTabId = FName(TEXT("MergeMyBluerpintTab"));
const FName MergeGraphTabId = FName(TEXT("MergeGraphTab"));

struct FBlueprintRevPair
{
	FBlueprintRevPair(const UBlueprint* InBlueprint, const FRevisionInfo& InRevData)
	: Blueprint(InBlueprint)
	, RevData(InRevData)
	{
	}

	const UBlueprint* Blueprint;
	const FRevisionInfo& RevData;
};;

static UEdGraph* FindGraphByPath(UBlueprint const& FromBlueprint, const FString& GraphPath)
{
	TArray<UEdGraph*> Graphs;
	FromBlueprint.GetAllGraphs(Graphs);

	for (UEdGraph* Graph : Graphs)
	{
		FString SearchGraphPath = FGraphDiffControl::GetGraphPath(Graph);
		if (SearchGraphPath.Equals(GraphPath))
		{
			return Graph;
		}
	}
	return nullptr;
}

struct FMergeGraphRowEntry
{
	FText Label;

	FString GraphPath;

	UEdGraphNode* LocalNode;
	UEdGraphNode* BaseNode;
	UEdGraphNode* RemoteNode;

	UEdGraphPin* LocalPin;
	UEdGraphPin* BasePin;
	UEdGraphPin* RemotePin;

	FLinearColor DisplayColor;

	bool bHasConflicts;
};

struct FMergeGraphEntry
{
	FString GraphPath;

	TArray<FMergeGraphRowEntry> Changes;
	bool bAnyConflics;
	bool bRemoteDifferences;
	bool bLocalDifferences;

	bool bExistsInRemote;
	bool bExistsInBase;
	bool bExistsInLocal;
};

static TArray< FMergeGraphEntry > GenerateDiffListItems(const FBlueprintRevPair& RemoteBlueprint, const FBlueprintRevPair& BaseBlueprint, const FBlueprintRevPair& LocalBlueprint )
{
	// Index all the graphs by name, we use the name of the graph as the 
	// basis of comparison between the various versions of the blueprint.
	TMap< FString, UEdGraph* > RemoteGraphMap, BaseGraphMap, LocalGraphMap;
	// We also want the set of all graph names in these blueprints, so that we 
	// can iterate over every graph.
	TSet< FString > AllGraphPaths;
	{
		TArray<UEdGraph*> GraphsRemote, GraphsBase, GraphsLocal;
		RemoteBlueprint.Blueprint->GetAllGraphs(GraphsRemote);
		BaseBlueprint.Blueprint->GetAllGraphs(GraphsBase);
		LocalBlueprint.Blueprint->GetAllGraphs(GraphsLocal);

		const auto ToMap = [&AllGraphPaths](const TArray<UEdGraph*>& InList, TMap<FString, UEdGraph*>& OutMap)
		{
			for (UEdGraph* Graph : InList)
			{
				FString GraphPath = FGraphDiffControl::GetGraphPath(Graph);

				OutMap.Add(GraphPath, Graph);
				AllGraphPaths.Add(GraphPath);
			}
		};
		ToMap(GraphsRemote, RemoteGraphMap);
		ToMap(GraphsBase, BaseGraphMap);
		ToMap(GraphsLocal, LocalGraphMap);
	}

	TArray< FMergeGraphEntry > Ret;
	{
		const auto GenerateDifferences = [](UEdGraph* GraphNew, UEdGraph** GraphOld)
		{
			TArray<FDiffSingleResult> Results;
			FGraphDiffControl::DiffGraphs(GraphOld ? *GraphOld : nullptr, GraphNew, Results);

			Algo::SortBy(Results, [](const FDiffResultItem& Data) { return Data.Result.Diff; });
			return Results;
		};

		for (const FString& GraphPath : AllGraphPaths)
		{
			TArray< FDiffSingleResult > RemoteDifferences;
			TArray< FDiffSingleResult > LocalDifferences;
			bool bExistsInRemote, bExistsInBase, bExistsInLocal;

			FMergeGraphEntry GraphEntry;
			GraphEntry.GraphPath = GraphPath;
			{
				UEdGraph** RemoteGraph = RemoteGraphMap.Find(GraphPath);
				UEdGraph** BaseGraph = BaseGraphMap.Find(GraphPath);
				UEdGraph** LocalGraph = LocalGraphMap.Find(GraphPath);

				GraphEntry.bAnyConflics = false;
				GraphEntry.bExistsInRemote = RemoteGraph != nullptr;
				GraphEntry.bExistsInBase = BaseGraph != nullptr;
				GraphEntry.bExistsInLocal = LocalGraph != nullptr;

				if (RemoteGraph)
				{
					RemoteDifferences = GenerateDifferences(*RemoteGraph, BaseGraph);
				}

				if (LocalGraph)
				{
					LocalDifferences = GenerateDifferences(*LocalGraph, BaseGraph);
				}

				// 'join' the local differences and remote differences by noting changes
				// that affected the same common base:
				{
					TMap< const FDiffSingleResult*, const FDiffSingleResult*> ConflictMap;

					for (const auto& RemoteDifference : RemoteDifferences)
					{
						const FDiffSingleResult* ConflictingDifference = nullptr;

						for (const auto& LocalDifference : LocalDifferences)
						{
							if (RemoteDifference.Node1 == LocalDifference.Node1)
							{
								if (RemoteDifference.Diff == EDiffType::NODE_REMOVED ||
									LocalDifference.Diff == EDiffType::NODE_REMOVED ||
									RemoteDifference.Pin1 == LocalDifference.Pin1)
								{
									ConflictingDifference = &LocalDifference;
									break;
								}
							}
							else if (RemoteDifference.Pin1 != nullptr && (RemoteDifference.Pin1 == LocalDifference.Pin1))
							{
								// it's possible the users made the same change to the same pin, but given the wide
								// variety of changes that can be made to a pin it is difficult to identify the change 
								// as identical, for now I'm just flagging all changes to the same pin as a conflict:
								ConflictingDifference = &LocalDifference;
								break;
							}
						}

						if (ConflictingDifference != nullptr)
						{
							// For now, we don't want to create a hard conflict for changes that don't effect runtime behavior:
							if (RemoteDifference.Diff == EDiffType::NODE_MOVED ||
								RemoteDifference.Diff == EDiffType::NODE_COMMENT)
							{
								continue;
							}

							ConflictMap.Add(&RemoteDifference, ConflictingDifference);
							ConflictMap.Add(ConflictingDifference, &RemoteDifference);
						}
					}


					for( const auto& Difference : RemoteDifferences )
					{
						FText Label;

						const FDiffSingleResult** ConflictingDifference = ConflictMap.Find(&Difference);

						if( ConflictingDifference )
						{
							Label = FText::Format( NSLOCTEXT("SMergeGraphView", "ConflictIdentifier", "CONFLICT: {0} conflicts with {1}" ), (*ConflictingDifference)->DisplayString, Difference.DisplayString );
						}
						else
						{
							Label = Difference.DisplayString;
						}

						FMergeGraphRowEntry NewEntry = {
							Label
							, Difference.OwningObjectPath
							, ConflictingDifference ? (*ConflictingDifference)->Node2 : nullptr /*UEdGraphNode* LocalNode*/
							, Difference.Node1 /*UEdGraphNode* BaseNode*/
							, Difference.Node2 /*UEdGraphNode* RemoteNode*/
							, ConflictingDifference ? (*ConflictingDifference)->Pin2 : nullptr /*UEdGraphPin* LocalPin*/
							, Difference.Pin1 /*UEdGraphPin* BasePin*/
							, Difference.Pin2 /*UEdGraphPin* RemotePin*/
							, Difference.GetDisplayColor()
							, ConflictingDifference ? true : false
						};

						GraphEntry.bAnyConflics |= NewEntry.bHasConflicts;
						GraphEntry.Changes.Push( NewEntry );
					}

					for (const auto& Difference : LocalDifferences)
					{
						FText Label;

						const FDiffSingleResult** ConflictingDifference = ConflictMap.Find( &Difference );

						if (!ConflictingDifference)
						{
							FMergeGraphRowEntry NewEntry = {
								Difference.DisplayString
								, Difference.OwningObjectPath
								, Difference.Node2 /*UEdGraphNode* LocalNode*/
								, Difference.Node1 /*UEdGraphNode* BaseNode*/
								, nullptr
								, Difference.Pin2 /*UEdGraphPin* LocalPin*/
								, Difference.Pin1 /*UEdGraphPin* BasePin*/
								, nullptr
								, Difference.GetDisplayColor()
								, false
							};

							GraphEntry.Changes.Push(NewEntry);
						}
					}

					GraphEntry.bLocalDifferences = LocalDifferences.Num() != 0;
					GraphEntry.bRemoteDifferences = RemoteDifferences.Num() != 0;
				}

				Ret.Add(GraphEntry);

				bExistsInRemote = RemoteGraph != nullptr;
				bExistsInBase = BaseGraph != nullptr;
				bExistsInLocal = LocalGraph != nullptr;
			}
		}
	}

	return Ret;
}

static void LockViews(TArray<FDiffPanel>& Views, bool bAreLocked)
{
	for (auto& Panel : Views)
	{
		auto GraphEditor = Panel.GraphEditor.Pin();
		if (GraphEditor.IsValid())
		{
			// lock this panel to ever other panel:
			for (auto& OtherPanel : Views)
			{
				auto OtherGraphEditor = OtherPanel.GraphEditor.Pin();
				if (OtherGraphEditor.IsValid() &&
					OtherGraphEditor != GraphEditor)
				{
					if (bAreLocked)
					{
						GraphEditor->LockToGraphEditor(OtherGraphEditor);
					}
					else
					{
						GraphEditor->UnlockFromGraphEditor(OtherGraphEditor);
					}
				}
			}
		}
	}
}

FDiffPanel& GetDiffPanelForNode(const UEdGraphNode& Node, TArray< FDiffPanel >& Panels)
{
	for (auto& Panel : Panels)
	{
		auto GraphEditor = Panel.GraphEditor.Pin();
		if (GraphEditor.IsValid())
		{
			if (Node.GetGraph() == GraphEditor->GetCurrentGraph())
			{
				return Panel;
			}
		}
	}
	checkf(false, TEXT("Looking for node %s but it cannot be found in provided panels"), *Node.GetName());
	static FDiffPanel Default;
	return Default;
}

void SMergeGraphView::Construct(const FArguments InArgs
	, const FBlueprintMergeData& InData
	, FOnMergeNodeSelected SelectionCallback
	, TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >& OutTreeEntries
	, TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >& OutRealDifferences
	, TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >& OutConflicts
	)
{
	const TSharedRef<SDockTab> MajorTab = SNew(SDockTab)
		.TabRole(ETabRole::MajorTab);

	TabManager = FGlobalTabmanager::Get()->NewTabManager(MajorTab);

	TabManager->RegisterTabSpawner(MergeGraphTabId,
		FOnSpawnTab::CreateRaw(this, &SMergeGraphView::CreateGraphDiffViews))
		.SetDisplayName(LOCTEXT("MergeGraphsTabTitle", "Graphs"))
		.SetTooltipText(LOCTEXT("MergeGraphsTooltipText", "Differences in the various graphs present in the blueprint"));

	TabManager->RegisterTabSpawner(MergeMyBluerpintTabId,
		FOnSpawnTab::CreateRaw(this, &SMergeGraphView::CreateMyBlueprintsViews))
		.SetDisplayName(LOCTEXT("MergeMyBlueprintTabTitle", "My Blueprint"))
		.SetTooltipText(LOCTEXT("MergeMyBlueprintTooltipText", "Differences in the 'My Blueprints' attributes of the blueprint"));

	Data = InData;
	bViewsAreLocked = true;
	
	TArray<FBlueprintRevPair> BlueprintsForDisplay;
	// EMergeParticipant::Remote
	BlueprintsForDisplay.Add(FBlueprintRevPair(InData.BlueprintRemote, InData.RevisionRemote));
	// EMergeParticipant::Base
	BlueprintsForDisplay.Add(FBlueprintRevPair(InData.BlueprintBase, InData.RevisionBase));
	// EMergeParticipant::Local
	BlueprintsForDisplay.Add(FBlueprintRevPair(InData.BlueprintLocal, FRevisionInfo()));
	
	const TSharedRef<FTabManager::FLayout> DefaultLayout = FTabManager::NewLayout("BlueprintMerge_Layout_v1")
	->AddArea
	(
		FTabManager::NewPrimaryArea()
		->Split
		(
			FTabManager::NewStack()
			->AddTab(MergeMyBluerpintTabId, ETabState::OpenedTab)
			->AddTab(MergeGraphTabId, ETabState::OpenedTab)
		)
	);

	for (int32 i = 0; i < EMergeParticipant::Max_None; ++i)
	{
		DiffPanels.Add(FDiffPanel());
		FDiffPanel& NewPanel = DiffPanels[i];
		NewPanel.Blueprint = BlueprintsForDisplay[i].Blueprint;
		NewPanel.RevisionInfo = BlueprintsForDisplay[i].RevData;
		NewPanel.bShowAssetName = false;
	}

	auto GraphPanelContainer = TabManager->RestoreFrom(DefaultLayout, TSharedPtr<SWindow>()).ToSharedRef();

	for (auto& Panel : DiffPanels )
	{
		Panel.InitializeDiffPanel();
	}

	auto DetailsPanelContainer = SNew(SSplitter);
	for( auto& Panel : DiffPanels )
	{
		DetailsPanelContainer->AddSlot()
		[
			Panel.GetDetailsWidget()
		];
	}

	Differences = TSharedPtr< TArray< FMergeGraphEntry > >( new TArray< FMergeGraphEntry >( 
																GenerateDiffListItems(BlueprintsForDisplay[EMergeParticipant::Remote]
																					, BlueprintsForDisplay[EMergeParticipant::Base]
																					, BlueprintsForDisplay[EMergeParticipant::Local]) ) );

	for( const auto& Difference : *Differences)
	{
		TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> > Children;
		for( const auto& Change : Difference.Changes )
		{
			const auto ChangeWidget = [](FText Label, FLinearColor Color) -> TSharedRef<SWidget>
			{
				return SNew(STextBlock)
						.Text(Label)
						.ColorAndOpacity(Color);
			};

			const auto SelectGraphNode = [](FOnMergeNodeSelected InSelectionCallback, FMergeGraphRowEntry DiffEntry, SMergeGraphView* Parent)
			{
				InSelectionCallback.ExecuteIfBound();
				Parent->HighlightEntry( DiffEntry );
			};

			TSharedPtr<FBlueprintDifferenceTreeEntry> Entry = TSharedPtr<FBlueprintDifferenceTreeEntry>(new FBlueprintDifferenceTreeEntry(
				FOnDiffEntryFocused::CreateStatic(SelectGraphNode, SelectionCallback, Change, this)
				, FGenerateDiffEntryWidget::CreateStatic(ChangeWidget, Change.Label, Change.DisplayColor )
				, TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >()
			));
			Children.Push(Entry);
			OutRealDifferences.Push(Entry);
			if( Change.LocalNode && Change.RemoteNode )
			{
				OutConflicts.Push(Entry);
			}
		}

		const auto Widget = []( const FMergeGraphEntry* InDifference ) -> TSharedRef<SWidget>
		{
			// blue indicates added, red indicates changed, yellow indicates removed, white indicates no change:
			const auto ComputeColor = [](const bool bAnyConflicts, const bool bAnyDifferences) -> FLinearColor
			{
				if( bAnyConflicts )
				{
					return DiffViewUtils::Conflicting();
				}
				else if( bAnyDifferences )
				{
					return DiffViewUtils::Differs();
				}
				return DiffViewUtils::Identical();
			};

			const auto Box = [](bool bIsPresent, FLinearColor Color) -> SHorizontalBox::FSlot::FSlotArguments
			{
				return MoveTemp(SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					.MaxWidth(8.0f)
					[
						SNew(SImage)
						.ColorAndOpacity(Color)
						.Image(bIsPresent ? FAppStyle::GetBrush("BlueprintDif.HasGraph") : FAppStyle::GetBrush("BlueprintDif.MissingGraph"))
					]);
			};

			FLinearColor RemoteColor = ComputeColor(InDifference->bAnyConflics, InDifference->bRemoteDifferences);
			FLinearColor BaseColor = ComputeColor(InDifference->bAnyConflics, false);
			FLinearColor LocalColor = ComputeColor(InDifference->bAnyConflics, InDifference->bLocalDifferences);
			FLinearColor TextColor = ComputeColor(InDifference->bAnyConflics, InDifference->bLocalDifferences || InDifference->bRemoteDifferences);

			FString DisplayString = InDifference->GraphPath;
			int32 PeriodIndex = INDEX_NONE;
			if (DisplayString.FindLastChar('.', PeriodIndex))
			{
				DisplayString.MidInline(PeriodIndex + 1, MAX_int32, EAllowShrinking::No);
			}

			return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				[
					SNew(STextBlock)
					.ColorAndOpacity( TextColor )
					.Text(FText::FromString(DisplayString))
				]
				+ DiffViewUtils::Box(InDifference->bExistsInRemote, RemoteColor)
				+ DiffViewUtils::Box(InDifference->bExistsInBase, BaseColor)
				+ DiffViewUtils::Box(InDifference->bExistsInLocal, LocalColor);
		};

		const auto FocusGraph = [](FOnMergeNodeSelected InSelectionCallback, SMergeGraphView* Parent, FString GraphPath)
		{
			InSelectionCallback.ExecuteIfBound();
			Parent->FocusGraph( GraphPath );
		};

		if( Children.Num() == 0 )
		{
			Children.Push(FBlueprintDifferenceTreeEntry::NoDifferencesEntry());
		}

		OutTreeEntries.Push(
			TSharedPtr<FBlueprintDifferenceTreeEntry>(new FBlueprintDifferenceTreeEntry(
				FOnDiffEntryFocused::CreateStatic(FocusGraph, SelectionCallback, this, Difference.GraphPath)
				, FGenerateDiffEntryWidget::CreateStatic(Widget, &Difference)
				, Children
			))
		);
	}

	ChildSlot
	[
		SNew(SSplitter)
		.Orientation(Orient_Horizontal)
		+ SSplitter::Slot()
		.Value(0.9f)
		[
			SNew(SSplitter)
			.Orientation(Orient_Vertical)
			+ SSplitter::Slot()
			.Value(.8f)
			[
				GraphPanelContainer
			]
			+ SSplitter::Slot()
			.Value(.2f)
			[
				DetailsPanelContainer
			]
		]
	];
}

void SMergeGraphView::FocusGraph(const FString& GraphPath)
{
	UEdGraph* GraphRemote = FindGraphByPath(*GetRemotePanel().Blueprint, GraphPath);
	UEdGraph* GraphBase = FindGraphByPath(*GetBasePanel().Blueprint, GraphPath);
	UEdGraph* GraphLocal = FindGraphByPath(*GetLocalPanel().Blueprint, GraphPath);

	GetBasePanel().GeneratePanel(GraphBase, nullptr);
	GetRemotePanel().GeneratePanel(GraphRemote, GraphBase);
	GetLocalPanel().GeneratePanel(GraphLocal, GraphBase);

	LockViews(DiffPanels, bViewsAreLocked);
}

void SMergeGraphView::HighlightEntry(const struct FMergeGraphRowEntry& Conflict)
{
	FocusGraph(Conflict.GraphPath);

	const auto FocusPinOrNode = [this]( UEdGraphPin* Pin, UEdGraphNode* Node )
	{
		if (Pin)
		{
			// then look for the diff panel and focus on the change:
			GetDiffPanelForNode(*Pin->GetOwningNode(), DiffPanels).FocusDiff(*Pin);
		}
		else if (Node)
		{
			GetDiffPanelForNode(*Node, DiffPanels).FocusDiff(*Node);
		}
	};

	// highlight the change made to the remote graph:
	FocusPinOrNode(Conflict.RemotePin, Conflict.RemoteNode);
	FocusPinOrNode(Conflict.LocalPin, Conflict.LocalNode);
	FocusPinOrNode(Conflict.BasePin, Conflict.BaseNode);
}

TSharedRef<SDockTab> SMergeGraphView::CreateGraphDiffViews(const FSpawnTabArgs& Args)
{
	auto PanelContainer = SNew(SSplitter);
	for (auto& Panel : DiffPanels)
	{
		PanelContainer->AddSlot()
		[
			SAssignNew(Panel.GraphEditorBox, SBox)
			.VAlign(VAlign_Fill)
			[
				SBlueprintDiff::DefaultEmptyPanel()
			]
		];
	}

	return SNew(SDockTab)
	[
		PanelContainer
	];
}

TSharedRef<SDockTab> SMergeGraphView::CreateMyBlueprintsViews(const FSpawnTabArgs& Args)
{
	auto PanelContainer = SNew(SSplitter);
	for (auto& Panel : DiffPanels)
	{
		PanelContainer->AddSlot()
		[
			Panel.GenerateMyBlueprintWidget()
		];
	}

	return SNew(SDockTab)
	[
		PanelContainer
	];
}

FReply SMergeGraphView::OnToggleLockView()
{
	bViewsAreLocked = !bViewsAreLocked;

	LockViews(DiffPanels, bViewsAreLocked);
	return FReply::Handled();
}

const FSlateBrush* SMergeGraphView::GetLockViewImage() const
{
	return bViewsAreLocked ? FAppStyle::GetBrush("Icons.Lock") : FAppStyle::GetBrush("Icons.Unlock");
}

#undef LOCTEXT_NAMESPACE
