// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"
#include "BlueprintMergeData.h"
#include "SBlueprintDiff.h"

class FSpawnTabArgs;
class FTabManager;

class SMergeGraphView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMergeGraphView){}
	SLATE_END_ARGS()

	void Construct(	const FArguments InArgs
					, const FBlueprintMergeData& InData
					, FOnMergeNodeSelected SelectionCallback
					, TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >& OutTreeEntries
					, TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >& OutRealDifferences
					, TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >& OutConflicts);
private:
	/** Helper functions and event handlers: */
	void FocusGraph(const FString& GraphPath);
	void HighlightEntry(const struct FMergeGraphRowEntry& Conflict);
	bool HasNoDifferences() const;
	
	TSharedRef<SDockTab> CreateGraphDiffViews(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> CreateMyBlueprintsViews(const FSpawnTabArgs& Args);

	FDiffPanel& GetRemotePanel() { return DiffPanels[EMergeParticipant::Remote]; }
	FDiffPanel& GetBasePanel() { return DiffPanels[EMergeParticipant::Base]; }
	FDiffPanel& GetLocalPanel() { return DiffPanels[EMergeParticipant::Local]; }

	FReply OnToggleLockView();
	const FSlateBrush*  GetLockViewImage() const;

	TArray< FDiffPanel > DiffPanels;
	FBlueprintMergeData Data;

	TSharedPtr< TArray< struct FMergeGraphEntry > > Differences;

	bool bViewsAreLocked;

	/** We can't use the global tab manager because we need to instance the merge control, so we have our own tab manager: */
	TSharedPtr<FTabManager> TabManager;
};
