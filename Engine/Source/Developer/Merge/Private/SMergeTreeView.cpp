// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMergeTreeView.h"
#include "Widgets/Layout/SSplitter.h"

#include "SCSDiff.h"

void SMergeTreeView::Construct(const FArguments InArgs
	, const FBlueprintMergeData& InData
	, FOnMergeNodeSelected SelectionCallback
	, TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >& OutTreeEntries
	, TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >& OutRealDifferences
	, TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >& OutConflicts)
{
	Data = InData;
	CurrentDifference = -1;
	CurrentMergeConflict = -1;

	// generate controls:
	// EMergeParticipant::Remote
	{
		SCSViews.Push(
			MakeShareable(new FSCSDiff(InData.BlueprintRemote))
			);
	}
	// EMergeParticipant::Base
	{
		SCSViews.Push(
			MakeShareable(new FSCSDiff(InData.BlueprintBase))
			);
	}
	// EMergeParticipant::Local
	{
		SCSViews.Push(
			MakeShareable(new FSCSDiff(InData.BlueprintLocal))
			);
	}

	TArray< FSCSResolvedIdentifier > RemoteHierarchy = GetRemoteView()->GetDisplayedHierarchy();
	TArray< FSCSResolvedIdentifier > BaseHierarchy = GetBaseView()->GetDisplayedHierarchy();
	TArray< FSCSResolvedIdentifier > LocalHierarchy = GetLocalView()->GetDisplayedHierarchy();

	FSCSDiffRoot RemoteDifferingProperties;
	DiffUtils::CompareUnrelatedSCS(InData.BlueprintBase, BaseHierarchy, InData.BlueprintRemote, RemoteHierarchy, RemoteDifferingProperties );
	FSCSDiffRoot LocalDifferingProperties;
	DiffUtils::CompareUnrelatedSCS(InData.BlueprintBase, BaseHierarchy, InData.BlueprintLocal, LocalHierarchy, LocalDifferingProperties);

	DifferingProperties = RemoteDifferingProperties;
	DifferingProperties.Entries.Append( LocalDifferingProperties.Entries );

	struct FSCSDiffPair
	{
		const FSCSDiffEntry* Local;
		const FSCSDiffEntry* Remote;
	};
	TArray < FSCSDiffPair > ConflictingDifferences;
	
	/* This predicate sorts the list of differing properties so that those that are 'earlier' in the tree appear first.
		For example, if we get the following two trees back:
			
			B added at position (3, 2, 1)
			C removed at position (1, 2)

			and 

			D added at position (4, 2, 1)

			the resulting list will be
			[C, B, D]: */
	const auto SortTreePredicate = []( const FSCSDiffEntry& A, const FSCSDiffEntry& B )
	{
		int32 Idx = 0;
		const TArray<int32>& ATreeAddress = A.TreeIdentifier.TreeLocation;
		const TArray<int32>& BTreeAddress = B.TreeIdentifier.TreeLocation;
		while(true)
		{
			if( !ATreeAddress.IsValidIndex(Idx) )
			{
				// A has a shorter address, show it first:
				return true;
			}
			else if( !BTreeAddress.IsValidIndex(Idx) )
			{
				// B has a shorter address, show it first:
				return false;
			}
			else if( ATreeAddress[Idx] < BTreeAddress[Idx] )
			{
				// A has a lower index, show it first:
				return true;
			}
			else if( ATreeAddress[Idx] > BTreeAddress[Idx] )
			{
				// B has a lower index, show it first:
				return false;
			}
			else
			{
				// tie, go to the next level of the tree:
				++Idx;
			}
		}
	};

	RemoteDifferingProperties.Entries.Sort(SortTreePredicate);
	LocalDifferingProperties.Entries.Sort(SortTreePredicate);

	const FText RemoteLabel = NSLOCTEXT("SMergeTreeView", "RemoteLabel", "Remote");
	const FText LocalLabel = NSLOCTEXT("SMergeTreeView", "LocalLabel", "Local");

	struct FSCSMergeEntry
	{
		FText Label;
		FSCSIdentifier Identifier;
		FPropertySoftPath PropertyIdentifier;
		bool bConflicted;
	};

	TArray<FSCSMergeEntry> Entries;
	bool bAnyConflict = false;
	int RemoteIter = 0;
	int LocalIter = 0;
	while(	RemoteIter != RemoteDifferingProperties.Entries.Num() ||
			LocalIter != LocalDifferingProperties.Entries.Num() )
	{
		if (RemoteIter != RemoteDifferingProperties.Entries.Num() &&
			LocalIter != LocalDifferingProperties.Entries.Num())
		{
			// check for conflicts:
			const FSCSDiffEntry& Remote = RemoteDifferingProperties.Entries[RemoteIter];
			const FSCSDiffEntry& Local = LocalDifferingProperties.Entries[LocalIter];
			if( Remote.TreeIdentifier == Local.TreeIdentifier)
			{
				bool bConflicting = true;

				if( Remote.DiffType == ETreeDiffType::NODE_PROPERTY_CHANGED &&
					Local.DiffType == ETreeDiffType::NODE_PROPERTY_CHANGED )
				{
					// conflict only if property changed is the same:
					bConflicting = Remote.PropertyDiff.Identifier == Local.PropertyDiff.Identifier;
				}

				if( bConflicting )
				{
					bAnyConflict = true;

					FSCSMergeEntry Entry = 
					{ 
						FText::Format(NSLOCTEXT("SMergeTreeView", "ConflictIdentifier", "CONFLICT: {0} conflicts with {1}"), DiffViewUtils::SCSDiffMessage(Remote, RemoteLabel), DiffViewUtils::SCSDiffMessage(Local, LocalLabel)),
						Remote.TreeIdentifier,
						Remote.DiffType == ETreeDiffType::NODE_PROPERTY_CHANGED ? Remote.PropertyDiff.Identifier : Local.PropertyDiff.Identifier,
						true 
					};

					// create a tree entry that describes both the local and remote change..
					Entries.Push( Entry );

					++RemoteIter;
					++LocalIter;
					continue;
				}
			}
		}

		// no possibility of conflict, advance the entry that has a 'lower' tree identifier, keeping in mind that tree identifier
		// may be equal, and that in that case we need to use the property identifier as a tiebreaker:
		const FSCSDiffEntry* Remote = RemoteIter != RemoteDifferingProperties.Entries.Num() ? &RemoteDifferingProperties.Entries[RemoteIter] : nullptr;
		const FSCSDiffEntry* Local = LocalIter != LocalDifferingProperties.Entries.Num() ? &LocalDifferingProperties.Entries[LocalIter] : nullptr;

		if( Local && ( !Remote || SortTreePredicate( *Local, *Remote ) ) )
		{
			FSCSMergeEntry Entry =
			{ 
				DiffViewUtils::SCSDiffMessage(*Local, LocalLabel),
				Local->TreeIdentifier,
				Local->PropertyDiff.Identifier,
				false 
			};
			Entries.Push( Entry );

			++LocalIter;
		}
		else
		{
			FSCSMergeEntry Entry =
			{ 
				DiffViewUtils::SCSDiffMessage(*Remote, RemoteLabel),
				Remote->TreeIdentifier,
				Remote->PropertyDiff.Identifier,
				false 
			};
			Entries.Push( Entry );

			++RemoteIter;
		}
	}

	const auto CreateSCSMergeWidget = [](FSCSMergeEntry Entry) -> TSharedRef<SWidget>
	{
		return SNew(STextBlock)
			.Text(Entry.Label)
			.ColorAndOpacity(Entry.bConflicted ? DiffViewUtils::Conflicting() : DiffViewUtils::Differs());
	};

	const auto FocusSCSDifferenceEntry = [](FSCSMergeEntry Entry, SMergeTreeView* Parent, FOnMergeNodeSelected InSelectionCallback)
	{
		InSelectionCallback.ExecuteIfBound();
		Parent->HighlightDifference(Entry.Identifier, Entry.PropertyIdentifier);
	};
	
	TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> > Children;
	for ( const auto& Difference : Entries )
	{
		auto Entry = TSharedPtr<FBlueprintDifferenceTreeEntry>(
			new FBlueprintDifferenceTreeEntry(
				FOnDiffEntryFocused::CreateStatic(FocusSCSDifferenceEntry, Difference, this, SelectionCallback)
				, FGenerateDiffEntryWidget::CreateStatic(CreateSCSMergeWidget, Difference)
				, TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >()
			)
		);
		Children.Push(Entry);
		OutRealDifferences.Push(Entry);
		if( Difference.bConflicted )
		{
			OutConflicts.Push(Entry);
		}
	}

	DifferingProperties.Entries.Sort(SortTreePredicate);

	const auto ForwardSelection = [](FOnMergeNodeSelected InSelectionCallback)
	{
		// This allows the owning control to focus the correct tab (or do whatever else it likes):
		InSelectionCallback.ExecuteIfBound();
	};

	const bool bHasDiffferences = Children.Num() != 0;
	if( !bHasDiffferences )
	{
		Children.Push( FBlueprintDifferenceTreeEntry::NoDifferencesEntry() );
	}

	TSharedPtr<FBlueprintDifferenceTreeEntry> Category = FBlueprintDifferenceTreeEntry::CreateCategoryEntryForMerge(
		NSLOCTEXT("FBlueprintDifferenceTreeEntry", "SCSLabel", "Components"),
		NSLOCTEXT("FBlueprintDifferenceTreeEntry", "SCSTooltip", "The list of changes made in the Components panel"),
		FOnDiffEntryFocused::CreateStatic(ForwardSelection, SelectionCallback),
		Children,
		RemoteDifferingProperties.Entries.Num() != 0,
		LocalDifferingProperties.Entries.Num() != 0,
		bAnyConflict);
	OutTreeEntries.Push(Category);

	ChildSlot[
		SNew(SSplitter)
			+ SSplitter::Slot()
			[
				GetRemoteView()->TreeWidget()
			]
		+ SSplitter::Slot()
			[
				GetBaseView()->TreeWidget()
			]
		+ SSplitter::Slot()
			[
				GetLocalView()->TreeWidget()
			]
	];
}

void SMergeTreeView::HighlightDifference(FSCSIdentifier TreeIdentifier, FPropertySoftPath Property)
{
	for (auto& View : SCSViews)
	{
		View->HighlightProperty(TreeIdentifier.Name, Property);
	}
}

TSharedRef<FSCSDiff>& SMergeTreeView::GetRemoteView()
{
	return SCSViews[EMergeParticipant::Remote];
}

TSharedRef<FSCSDiff>& SMergeTreeView::GetBaseView()
{
	return SCSViews[EMergeParticipant::Base];
}

TSharedRef<FSCSDiff>& SMergeTreeView::GetLocalView()
{
	return SCSViews[EMergeParticipant::Local];
}

