// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "BlueprintMergeData.h"
#include "DiffUtils.h"

class FSCSDiff;

class SMergeTreeView	: public SCompoundWidget
{
public:
	virtual ~SMergeTreeView() = default;

	SLATE_BEGIN_ARGS(SMergeTreeView)
	{}
	SLATE_END_ARGS()

	void Construct(	const FArguments InArgs
					, const FBlueprintMergeData& InData
					, FOnMergeNodeSelected SelectionCallback
					, TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >& OutTreeEntries
					, TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >& OutRealDifferences
					, TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >& OutConflicts);
private:
	void HighlightDifference( FSCSIdentifier VarName, FPropertySoftPath Property );

	TSharedRef<FSCSDiff>& GetRemoteView();
	TSharedRef<FSCSDiff>& GetBaseView();
	TSharedRef<FSCSDiff>& GetLocalView();

	FBlueprintMergeData Data;
	TArray< TSharedRef<FSCSDiff> > SCSViews;

	FSCSDiffRoot MergeConflicts;
	int CurrentMergeConflict;

	FSCSDiffRoot DifferingProperties;
	int CurrentDifference;
};
