// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class UBlueprint;

namespace EMergeResult
{
	enum Type
	{
		Unknown,
		Completed,
		Cancelled,
	};
}
DECLARE_DELEGATE_TwoParams(FOnMergeResolved, UPackage*, EMergeResult::Type);

/**
 * The public interface to this module
 */
class IMerge : public IModuleInterface
{

public:

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IMerge& Get()
	{
		return FModuleManager::LoadModuleChecked< IMerge >( "Merge" );
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded( "Merge" );
	}

	/**
	 * Generates a widget used to perform a three way merge operation.
	 *
	 * @return The merge view widget
	 */
	virtual TSharedPtr<class SDockTab> GenerateMergeWidget( const UBlueprint& Object, TSharedRef< class FBlueprintEditor> Editor ) = 0;
	virtual TSharedPtr<class SDockTab> GenerateMergeWidget(const UBlueprint* BaseBlueprint, const UBlueprint* RemoteBlueprint, const UBlueprint* LocalBlueprint, const FOnMergeResolved& MergeResolutionCallback, TSharedRef<class FBlueprintEditor> Editor) = 0;

	/** 
	 * @return whether the blueprint is in a conflicted state, and can therefore be merged.
	 */
	virtual bool PendingMerge(const UBlueprint& BlueprintObj) const = 0;
};

